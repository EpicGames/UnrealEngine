// Copyright Epic Games, Inc. All Rights Reserved.

#include "AgentSkillsTest.h"

#include "CoreMinimal.h"
#include "EditorAssetLibrary.h"
#include "Engine/Engine.h"
#include "FileHelpers.h"
#include "HAL/FileManager.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/AutomationTest.h"
#include "Misc/Guid.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "ObjectTools.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "Tests/ToolsetRegistryTestFlags.h"
#include "ToolsetRegistry/AgentSkill.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"


#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FAgentSkillSpec, "AI.ToolsetRegistry.AgentSkillSpec",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	FString MountPoint;
	FString SkillPath;
	FString DefaultDescription = FString(TEXT("Testy test"));
	FString DefaultInstructions = FString(TEXT("Do important testy things."));
	UAgentSkill* AgentSkill;
	UAgentSkill* PromptSkill;
	TArray<FString> SavedAgentSkillBlockedNames;
	TArray<FString> SavedAgentSkillAllowedNames;
END_DEFINE_SPEC(FAgentSkillSpec)

void FAgentSkillSpec::Define()
{
	BeforeEach([this]()
	{
		MountPoint = "/Automation/";
		FPackageName::RegisterMountPoint(*MountPoint, FPaths::AutomationTransientDir());

		FAgentSkillDetails Details;
		Details.Instructions = DefaultInstructions;
		SkillPath = UAgentSkillToolset::CreateSkill(
			MountPoint, FString(TEXT("Test")), DefaultDescription, Details);

		UClass* AgentSkillClass = LoadObject<UClass>(nullptr, SkillPath);
		AgentSkill = Cast<UAgentSkill>(AgentSkillClass->GetDefaultObject());

		PromptSkill = Cast<UAgentSkill>(
			UAgentSkillCustomPrompt::StaticClass()->GetDefaultObject());
		PromptSkill->Description = FString(TEXT("CustomPrompt:"));
	});

	AfterEach([this]()
	{
		PromptSkill->Description.Empty();
		PromptSkill = nullptr;
		AgentSkill = nullptr;
		SkillPath.Empty();
		GEngine->ForceGarbageCollection(true);
		UEditorAssetLibrary::DeleteAsset(FString::Printf(TEXT("%sTest"), *MountPoint));
		FPackageName::UnRegisterMountPoint(*MountPoint, FPaths::AutomationTransientDir());
		MountPoint.Empty();
	});

	It("Can create skills", [this]()
	{
		TestEqual(
			"Created", AgentSkill->Description, DefaultDescription);
		TestEqual(
			"Created", AgentSkill->Instructions, DefaultInstructions);
	});

	It("Won't create when given bad folder paths", [this]()
	{
		FAgentSkillDetails Details;
		TestEqual("BadPath", UAgentSkillToolset::CreateSkill(
			FString(), FString(), FString(), Details), FString());
	});

	It("Won't create when given bad asset paths", [this]()
	{
		FAgentSkillDetails Details;
		TestEqual("BadPath", UAgentSkillToolset::CreateSkill(
			MountPoint, FString(), FString(), Details), FString());
	});

	It("Won't create when given non-existant folders", [this]()
	{
		FAgentSkillDetails Details;
		TestEqual("BadPath", UAgentSkillToolset::CreateSkill(
			FString(TEXT("/Bad/Path")), FString(TEXT("Bad")), FString(), Details), FString());
	});

	It("Can update skills", [this]()
	{
		FAgentSkillDetails Details;
		Details.Instructions = FString(TEXT("Foo bar"));
		UAgentSkillToolset::UpdateSkill(SkillPath, FString(TEXT("Bish bosh")), Details);
		TestEqual(
			"Created", AgentSkill->Description, FString(TEXT("Bish bosh")));
		TestEqual(
			"Created", AgentSkill->Instructions, FString(TEXT("Foo bar")));
	});

	It("Won't update when class does not exist", [this]()
	{
		FAgentSkillDetails Details;
		TestFalse("BadPath", UAgentSkillToolset::UpdateSkill(
			FString(TEXT("Does/Not/Exist_C")), FString(), Details));
	});

	It("Won't update when class is not a skill", [this]()
	{
		FAgentSkillDetails Details;
		TestFalse("BadPath", UAgentSkillToolset::UpdateSkill(
			FString(TEXT("/Script/Engine.StaticMeshActor")), FString(), Details));
	});

	It("Won't update native (C++) skills", [this]()
	{
		UClass* NativeSkillClass = UAgentSkillCustomPrompt::StaticClass();
		TestTrue("Native skill class has CLASS_Native",
			NativeSkillClass->HasAnyClassFlags(CLASS_Native));

		const FString OriginalDescription = PromptSkill->Description;
		const FString OriginalInstructions = PromptSkill->Instructions;

		FAgentSkillDetails Details;
		Details.Instructions = FString(TEXT("Mutated instructions"));
		TestFalse("Refused", UAgentSkillToolset::UpdateSkill(
			NativeSkillClass->GetPathName(), FString(TEXT("Mutated description")), Details));

		TestEqual("Description unchanged", PromptSkill->Description, OriginalDescription);
		TestEqual("Instructions unchanged", PromptSkill->Instructions, OriginalInstructions);
	});

	It("Won't update transient skills", [this]()
	{
		UClass* TransientSkillClass = UAgentSkillTransient::StaticClass();
		TestTrue("Transient skill class has RF_Transient",
			TransientSkillClass->HasAnyFlags(RF_Transient));

		UAgentSkill* TransientSkill =
			Cast<UAgentSkill>(TransientSkillClass->GetDefaultObject());
		const FString OriginalDescription = TransientSkill->Description;
		const FString OriginalInstructions = TransientSkill->Instructions;

		FAgentSkillDetails Details;
		Details.Instructions = FString(TEXT("Mutated instructions"));
		TestFalse("Refused", UAgentSkillToolset::UpdateSkill(
			TransientSkillClass->GetPathName(), FString(TEXT("Mutated description")), Details));

		TestEqual("Description unchanged", TransientSkill->Description, OriginalDescription);
		TestEqual("Instructions unchanged", TransientSkill->Instructions, OriginalInstructions);
	});

	It("List skill summaries", [this]()
	{
		TMap<FString, FString> Skills = UAgentSkillToolset::ListSkills();

		// AgentSkill in a native class
		{
			FString* Description = Skills.Find(PromptSkill->GetClass()->GetPathName());
			TestTrue("Summary", Description != nullptr);
			if (Description)
			{
				TestEqual("Summary", *Description, PromptSkill->Description);
			}
		}
		// AgentSkill in BP class.
		{
			FString* Description = Skills.Find(AgentSkill->GetClass()->GetPathName());
			TestTrue("Summary", Description != nullptr);
			if (Description)
			{
				TestEqual("Summary", *Description, AgentSkill->Description);
			}
		}
	});

	It("Get skill details", [this]()
	{
		FString ClassName = AgentSkill->GetClass()->GetPathName();
		TMap<FString, FAgentSkillDetails> Summaries =
			UAgentSkillToolset::GetSkills(TArray<FString>({ ClassName }));
		FAgentSkillDetails* Details = Summaries.Find(ClassName);
		TestTrue("Details", Details != nullptr);
		if (Details)
		{
			TestEqual("Details", Details->Instructions, AgentSkill->Instructions);
		}
	});

	It("Can't load non-existant skills", [this]()
	{
		TMap<FString, FAgentSkillDetails> Summaries =
			UAgentSkillToolset::GetSkills(TArray<FString>({ "DoesNotExist" }));
		TestEqual("Details", Summaries.Num(), 0);
	});

	It("Can customize prompt geneation", [this]()
	{
		TObjectPtr<UAgentSkill> AgentSkillCustom = NewObject<UAgentSkillCustomPrompt>();
		AgentSkillCustom->Instructions = FString(TEXT("FooBar"));
		FAgentSkillDetails Details = AgentSkillCustom->GetDetails();
		TestEqual("Details", Details.Instructions, FString(TEXT("FooBarBishBosh")));
	});

	Describe("Filtering", [this]()
	{
		BeforeEach([this]()
		{
			UToolsetRegistrySettings* Settings = GetMutableDefault<UToolsetRegistrySettings>();
			SavedAgentSkillBlockedNames = Settings->AgentSkillBlockedNames;
			SavedAgentSkillAllowedNames = Settings->AgentSkillAllowedNames;
		});

		AfterEach([this]()
		{
			UToolsetRegistrySettings* Settings = GetMutableDefault<UToolsetRegistrySettings>();
			Settings->AgentSkillBlockedNames = SavedAgentSkillBlockedNames;
			Settings->AgentSkillAllowedNames = SavedAgentSkillAllowedNames;
			SavedAgentSkillBlockedNames.Empty();
			SavedAgentSkillAllowedNames.Empty();
		});

		It("AgentSkillBlockedNames hides skills from ListSkills via simple substring", [this]()
		{
			UToolsetRegistrySettings* Settings = GetMutableDefault<UToolsetRegistrySettings>();
			Settings->AgentSkillBlockedNames = { TEXT("/Automation/") };
			Settings->AgentSkillAllowedNames = {};

			TMap<FString, FString> Skills = UAgentSkillToolset::ListSkills();
			TestFalse("Blocked", Skills.Contains(AgentSkill->GetClass()->GetPathName()));
			TestTrue("Other present",
				Skills.Contains(PromptSkill->GetClass()->GetPathName()));
		});

		It("AgentSkillBlockedNames hides skills from GetSkills", [this]()
		{
			UToolsetRegistrySettings* Settings = GetMutableDefault<UToolsetRegistrySettings>();
			Settings->AgentSkillBlockedNames = { TEXT("Test") };
			Settings->AgentSkillAllowedNames = {};

			const FString ClassPath = AgentSkill->GetClass()->GetPathName();
			TMap<FString, FAgentSkillDetails> Skills =
				UAgentSkillToolset::GetSkills(TArray<FString>({ ClassPath }));
			TestEqual("Blocked", Skills.Num(), 0);
		});

		It("AgentSkillBlockedNames supports regex patterns", [this]()
		{
			UToolsetRegistrySettings* Settings = GetMutableDefault<UToolsetRegistrySettings>();
			Settings->AgentSkillBlockedNames = { TEXT("/Test_C$/") };
			Settings->AgentSkillAllowedNames = {};

			TMap<FString, FString> Skills = UAgentSkillToolset::ListSkills();
			TestFalse("Blocked", Skills.Contains(AgentSkill->GetClass()->GetPathName()));
		});

		It("AgentSkillAllowedNames restricts ListSkills to matching skills", [this]()
		{
			UToolsetRegistrySettings* Settings = GetMutableDefault<UToolsetRegistrySettings>();
			Settings->AgentSkillBlockedNames = {};
			Settings->AgentSkillAllowedNames = { TEXT("/Automation/") };

			TMap<FString, FString> Skills = UAgentSkillToolset::ListSkills();
			TestTrue("Allowed",
				Skills.Contains(AgentSkill->GetClass()->GetPathName()));
			TestFalse("Other hidden",
				Skills.Contains(PromptSkill->GetClass()->GetPathName()));
		});

		It("AgentSkillAllowedNames restricts GetSkills to matching skills", [this]()
		{
			UToolsetRegistrySettings* Settings = GetMutableDefault<UToolsetRegistrySettings>();
			Settings->AgentSkillBlockedNames = {};
			Settings->AgentSkillAllowedNames = { TEXT("Nothing matches this") };

			const FString ClassPath = AgentSkill->GetClass()->GetPathName();
			TMap<FString, FAgentSkillDetails> Skills =
				UAgentSkillToolset::GetSkills(TArray<FString>({ ClassPath }));
			TestEqual("Hidden", Skills.Num(), 0);
		});

		It("AgentSkillBlockedNames takes precedence over AgentSkillAllowedNames", [this]()
		{
			UToolsetRegistrySettings* Settings = GetMutableDefault<UToolsetRegistrySettings>();
			Settings->AgentSkillBlockedNames = { TEXT("Test") };
			Settings->AgentSkillAllowedNames = { TEXT("Test") };

			TMap<FString, FString> Skills = UAgentSkillToolset::ListSkills();
			TestFalse("Block wins",
				Skills.Contains(AgentSkill->GetClass()->GetPathName()));
		});
	});
}

#endif
