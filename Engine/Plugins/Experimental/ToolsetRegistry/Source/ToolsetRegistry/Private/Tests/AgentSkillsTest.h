// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolsetRegistry/AgentSkill.h"

#include "AgentSkillsTest.generated.h"

UCLASS(Hidden)
class UAgentSkillCustomPrompt : public UAgentSkill
{
	GENERATED_BODY()

protected:
	virtual FString GeneratePrompt_Implementation(const FString& InitialInstructions) const override
	{
		return InitialInstructions + FString(TEXT("BishBosh"));
	}
};

UCLASS(Hidden, Transient)
class UAgentSkillTransient : public UAgentSkill
{
	GENERATED_BODY()
};
