// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorConverterTest.h"

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#include "ToolsetRegistry/JsonSchema.h"
#include "ToolsetRegistry/ToolsetJson.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FToolsetColorConverterSpec, "AI.ToolsetRegistry.ColorConverterSpec",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	void TestColorSchema(const TSharedPtr<FJsonObject>& Schema, bool HasMax);
	void TestColorData(const TSharedPtr<FJsonObject>& Schema, const FColor& Expected);
END_DEFINE_SPEC(FToolsetColorConverterSpec)

void FToolsetColorConverterSpec::TestColorSchema(const TSharedPtr<FJsonObject>& Schema, bool HasMax)
{
	if (!TestTrue("Schema", Schema.IsValid())) return;

	FString Type;
	TestTrue("Type", Schema->TryGetStringField(TEXT("type"), Type));
	TestEqual("Type", Type, FString(TEXT("object")));

	FString Title;
	TestTrue("Title", Schema->TryGetStringField(TEXT("title"), Title));
	TestEqual("Title", Title, FString(TEXT("LinearColor")));

	TestFalse("Description", Schema->HasField(TEXT("description")));

	const TSharedPtr<FJsonObject>* Properties = nullptr;
	if (!(TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties)) &&
		Properties))
	{
		return;
	}
	TArray<FString> PropertyNames({ "r", "g", "b", "a" });
	for (const auto& Name : PropertyNames)
	{
		const TSharedPtr<FJsonObject>* Property = nullptr;
		if (!(TestTrue("Property", Properties->Get()->TryGetObjectField(Name, Property)) &&
			Property))
		{
			continue;
		}
		TestEqual("Property", Property->Get()->GetStringField(
			TEXT("type")), FString(TEXT("number")));
		TestEqual("Property", Property->Get()->GetNumberField(
			TEXT("minimum")), 0.0);
		if (HasMax)
		{
			TestEqual("Property", Property->Get()->GetNumberField(
				TEXT("maximum")), 1.0);
		}
	}
}

void FToolsetColorConverterSpec::TestColorData(
	const TSharedPtr<FJsonObject>& Schema, const FColor& Expected)
{
	if (!TestTrue("Schema", Schema.IsValid())) return;

	const FLinearColor LinearExpected = Expected.ReinterpretAsLinear();

	float R;
	TestTrue("Color", Schema->TryGetNumberField(TEXT("r"), R));
	TestEqual("Color", R, LinearExpected.R, 2e-3f);

	float G;
	TestTrue("Color", Schema->TryGetNumberField(TEXT("g"), G));
	TestEqual("Color", G, LinearExpected.G, 2e-3f);

	float B;
	TestTrue("Color", Schema->TryGetNumberField(TEXT("b"), B));
	TestEqual("Color", B, LinearExpected.B, 2e-3f);

	float A;
	TestTrue("Color", Schema->TryGetNumberField(TEXT("a"), A));
	TestEqual("Color", A, LinearExpected.A, 2e-3f);
}

void FToolsetColorConverterSpec::Define()
{
	using namespace UE::ToolsetRegistry::Internal;
	using namespace UE::ToolsetRegistry::Internal::ToolsetJson;

	It("Converts FColor property to FLinearColor schema", [this]()
	{
		UStruct* Struct = FColorConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestColor");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Prop);
		TestColorSchema(Schema, true);
	});

	It("Converts FColor property to and from JSON", [this]()
	{
		FColorConverterTest TestIn;
		TestIn.TestColor = FColor(10, 20, 30, 40);

		UStruct* Struct = FColorConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestColor");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestColor);
		TestColorData(OutJson->AsObject(), TestIn.TestColor);

		FColorConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestColor));
		TestEqual("ColorCheck", TestIn.TestColor, TestOut.TestColor);
	});

	It("Converts FLinearColor property to FLinearColor schema", [this]()
	{
		UStruct* Struct = FColorConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestLinearColor");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Prop);
		TestColorSchema(Schema, false);
	});

	It("Converts FLinearColor property to and from JSON", [this]()
	{
		FColorConverterTest TestIn;
		TestIn.TestLinearColor = FLinearColor(0.1f, 0.2f, 0.3f, 0.4f);

		UStruct* Struct = FColorConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestLinearColor");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestLinearColor);
		TestColorData(OutJson->AsObject(), TestIn.TestLinearColor.ToFColor(false));

		FColorConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestLinearColor));
		TestEqual("ColorCheck", TestIn.TestLinearColor, TestOut.TestLinearColor);
	});

	It("Can read color defaults", [this]()
	{
		UFunction* FunctionPtr = UColorConverterTestObject::StaticClass()->FindFunctionByName("TestDefault");
		const TSharedPtr<FJsonObject> Schema = StructToJsonSchema(FunctionPtr);
		if (!TestTrue("Schema", Schema.IsValid())) return;

		const TSharedPtr<FJsonObject>* InputSchema = nullptr;
		if (!(TestTrue("Input", Schema->TryGetObjectField(TEXT("inputSchema"), InputSchema)) &&
			InputSchema))
		{
			return;
		}

		TestFalse("NoRequired", InputSchema->Get()->HasField(TEXT("required")));

		const TSharedPtr<FJsonObject>* Properties = nullptr;
		if (!(TestTrue("Properties", InputSchema->Get()->TryGetObjectField(TEXT("properties"), Properties)) &&
			Properties))
		{
			return;
		}

		const TSharedPtr<FJsonObject>* TestColor = nullptr;
		if (!(TestTrue("testColor", Properties->Get()->TryGetObjectField(TEXT("testColor"), TestColor)) &&
			TestColor))
		{
			return;
		}
		TestColorData(TestColor->Get()->GetObjectField(TEXT("default")), FColor(10, 20, 30, 40));

		const TSharedPtr<FJsonObject>* TestLinearColor = nullptr;
		if (!(TestTrue("testLinearColor", Properties->Get()->TryGetObjectField(TEXT("testLinearColor"), TestLinearColor)) &&
			TestLinearColor))
		{
			return;
		}
		TestColorData(
			TestLinearColor->Get()->GetObjectField(TEXT("default")), FLinearColor(0.1f, 0.2f, 0.3f, 0.4f).ToFColor(false));
	});
}

#endif
