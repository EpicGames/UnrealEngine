// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ColorConverterTest.generated.h"


UCLASS(BlueprintType)
class UColorConverterTestObject : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category="ColorConverterTest")
	void TestDefault(
		FColor TestColor = FColor(10, 20, 30, 40),
		FLinearColor TestLinearColor = FLinearColor(0.1f, 0.2f, 0.3f, 0.4f)) {}
};

USTRUCT(BlueprintType)
struct FColorConverterTest
{
	GENERATED_BODY()
public:
	UPROPERTY()
	FColor TestColor = FColor::White;

	UPROPERTY()
	FLinearColor TestLinearColor = FLinearColor::White;
};
