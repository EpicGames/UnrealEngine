// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Delegates/DelegateCombinations.h"
#include "Misc/AutomationTest.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

#include "ToolsetRegistry/DelegateHandle.h"
#include "ToolsetRegistryTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace UE::ToolsetRegistry
{
	namespace
	{
		struct FCallCounter
		{
			int CallCount = 0;

			void Call()
			{
				CallCount++;
			}

			TFunction<void()> Callable()
			{
				return [this]() -> void { Call(); };
			}
		};
	}

	DECLARE_MULTICAST_DELEGATE(FToolsetRegistryOnTest)

	BEGIN_DEFINE_SPEC(FToolsetRegistryDelegateHandleRaii, "AI.ToolsetRegistry.DelegateHandleRaii",
		ToolsetRegistryTest::Flags)
		END_DEFINE_SPEC(FToolsetRegistryDelegateHandleRaii)

		void FToolsetRegistryDelegateHandleRaii::Define()
	{
		Describe(TEXT("TDelegateHandleRaii"), [this]
			{
				It(TEXT("can construct a container attached to a delegate handle"), [this]
					{
						FToolsetRegistryOnTest OnTest;
						FCallCounter CallCounter;
						auto RawDelegateHandle = OnTest.AddLambda(CallCounter.Callable());
						auto RawDelegateHandleToMove = RawDelegateHandle;
						TDelegateHandleRaii Handle(OnTest, MoveTemp(RawDelegateHandleToMove));
						(void)TestEqual(TEXT("Delegate handle"), Handle.Get(), RawDelegateHandle);
					});

				It(TEXT("can construct a container that references an invalid handle"), [this]
					{
						FToolsetRegistryOnTest OnTest;
						FDelegateHandle RawDelegateHandle;
						TDelegateHandleRaii Handle(OnTest, MoveTemp(RawDelegateHandle));
						(void)TestFalse(
							TEXT("Delegate handle is not valid"), Handle.Get().IsValid());
					});

				It(TEXT("should unregister a delegate handler on reset"), [this]
					{
						FToolsetRegistryOnTest OnTest;
						FCallCounter CallCounter;
						TDelegateHandleRaii Handle(
							OnTest, OnTest.AddLambda(CallCounter.Callable()));
						OnTest.Broadcast();
						(void)TestEqual(TEXT("Called once"), CallCounter.CallCount, 1);
						Handle.Reset();

						OnTest.Broadcast();
						(void)TestEqual(TEXT("Still called once"), CallCounter.CallCount, 1);
						(void)TestFalse(
							TEXT("Delegate handle is not valid"),
							Handle.Get().IsValid());
					});

				It(TEXT("should unregister a delegate handler on destruction"), [this]
					{
						FToolsetRegistryOnTest OnTest;
						FCallCounter CallCounter;
						{
							TDelegateHandleRaii Handle(
								OnTest, OnTest.AddLambda(CallCounter.Callable()));
							OnTest.Broadcast();
							(void)TestEqual(TEXT("Called once"), CallCounter.CallCount, 1);
						}

						OnTest.Broadcast();
						(void)TestEqual(TEXT("Still called once"), CallCounter.CallCount, 1);
					});
			});

		Describe(TEXT("FDelegateHandleRaii::Create"), [this]
			{
				It(TEXT("should create a container attached to a delegate handle"), [this]
					{
						FToolsetRegistryOnTest OnTest;
						FCallCounter CallCounter;
						auto RawDelegateHandle = OnTest.AddLambda(CallCounter.Callable());
						auto RawDelegateHandleToMove = RawDelegateHandle;
						auto Handle = FDelegateHandleRaii::Create(
							OnTest, MoveTemp(RawDelegateHandleToMove));
						if (!TestTrue(TEXT("Handle created"), Handle.IsValid())) return;
						(void)TestEqual(TEXT("Delegate handle"),
							Handle->Get(), RawDelegateHandle);
					});
			});

		Describe(TEXT("FDelegateHandleRaii::IsValid"), [this]
			{
				It(TEXT("should return false if it does not reference a valid pointer"), [this]
					{
						FDelegateHandleRaii Handle;
						(void)TestFalse(TEXT("Not valid on construction"), Handle.IsValid());
					});

				It(TEXT("should return true if it references a valid pointer and handle"), [this]
					{
						FToolsetRegistryOnTest OnTest;
						FCallCounter CallCounter;
						auto Handle = FDelegateHandleRaii::Create(
							OnTest, OnTest.AddLambda(CallCounter.Callable()));
						(void)TestTrue(TEXT("Valid after create"), Handle.IsValid());
					});

				It(TEXT("should return false if it references an invalid handle"), [this]
					{
						FToolsetRegistryOnTest OnTest;
						FCallCounter CallCounter;
						auto Handle = FDelegateHandleRaii::Create(
							OnTest, OnTest.AddLambda(CallCounter.Callable()));
						Handle->Reset();
						(void)TestFalse(TEXT("Not valid after reset"), Handle.IsValid());
					});
			});
	}

}
#endif  // WITH_DEV_AUTOMATION_TESTS
