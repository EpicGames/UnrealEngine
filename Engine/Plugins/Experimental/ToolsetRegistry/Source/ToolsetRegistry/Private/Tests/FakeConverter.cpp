// Copyright Epic Games, Inc. All Rights Reserved.

#include "FakeConverter.h"

#include "Templates/UnrealTemplate.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::ToolsetRegistry
{
	FFakeConverter::FFakeConverter() = default;

	FFakeConverter::~FFakeConverter() = default;

	FString FFakeConverter::GetName() const
	{
		static FString Name(TEXT("FakeConverter"));
		return Name;
	}
}

#endif