// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ToolsetRegistry/ToolsetJsonConverter.h"

#include "FakeConverter.generated.h"

USTRUCT(BlueprintType)
struct FFakeConverterTest
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category = "FakeConverter")
	float TestFloat = 0.f;

	UPROPERTY(BlueprintReadWrite, Category = "FakeConverter")
	FString TestString;
};

#if WITH_DEV_AUTOMATION_TESTS

#define UE_API TOOLSETREGISTRY_API

namespace UE::ToolsetRegistry
{

class FFakeConverter : public FToolsetJsonConverter
{
public:
	UE_API FFakeConverter();
	UE_API virtual ~FFakeConverter();

	/// Returns the name of the converter
	virtual FString GetName() const  override;

	virtual bool CanConvertProperty(TNotNull<const FProperty*> Property)  override
	{
		return CastField<FNumericProperty>(Property) != nullptr;
	}

	virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(
		TNotNull<const FProperty*> Property)  override
	{
		return nullptr;
	}

	virtual TSharedPtr<FJsonValue> PropertyToDefault(
		TNotNull<const FProperty*> Property, const FString& DefaultString)  override
	{
		return nullptr;
	}

	virtual TSharedPtr<FJsonValue> PropertyToJsonData(
		TNotNull<FProperty*> Property, const void* Value)  override
	{
		return nullptr;
	}

	virtual bool JsonDataToProperty(
		const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
		void* OutValue, UObject* Outer)  override
	{
		return false;
	}
};

}

#undef UE_API

#endif
