// Copyright Epic Games, Inc. All Rights Reserved.

#include "FakeToolset.h"

#include "Templates/UnrealTemplate.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::ToolsetRegistry
{
	FFakeToolset::FFakeToolset(const FString& Name) :
		Name(Name)
	{
		ToolPromises.Add(
			TEXT("SomeTool"),
			MakeShared<TPromise<TValueOrError<FString, FString>>>(
				MakeFulfilledPromise<TValueOrError<FString, FString>>(
					MakeValue(SUCCESSFUL_RESULT))));
	}

	FFakeToolset::~FFakeToolset() = default;

	TFuture<TValueOrError<FString, FString>> FFakeToolset::ExecuteToolInternal(
		const FString& ToolName, const FString& JsonInput)
	{
		TSharedPtr<TPromise<TValueOrError<FString, FString>>> Promise;
		if (!ToolPromises.RemoveAndCopyValue(ToolName, Promise) || !Promise)
		{
			Promise =
				MakeShared<TPromise<TValueOrError<FString, FString>>>(
					MakeFulfilledPromise<TValueOrError<FString, FString>>(
						MakeError(FString(TEXT("Tool not found")))));
		}
		return Promise->GetFuture();
	}

	FString FFakeToolset::GetJsonSchemaInternal() const
	{
		// Simplified schema for testing. Tools are represented as objects with a "name" field
		// matching the format used by FBlueprintLibraryToolset, so schema filtering works correctly.
		FString Tools;
		bool bFirst = true;
		for (const auto& [ToolName, ToolPromise] : ToolPromises)
		{
			if (!bFirst) Tools += ",";
			bFirst = false;
			Tools += FString::Printf(
				TEXT(R"json({"name":"%s.%s"})json"), *GetToolsetName(), *ToolName);
		}
		return FString::Printf(
			TEXT(R"json({"name":"%s","tools":[%s]})json"), *GetToolsetName(), *Tools);
	}

	FString FFakeToolset::GetToolsetName() const
	{
		return Name;
	}

	FString FFakeToolset::GetToolsetVersion() const
	{
		return TEXT("1.0");
	}

	FString FFakeToolset::GetToolsetDescription() const
	{
		return TEXT("Fake Toolset for testing.");
	}

	TSharedPtr<TPromise<TValueOrError<FString, FString>>> FFakeToolset::AddFakeToolCall(
		const FString& ToolName,
		TPromise<TValueOrError<FString, FString>>&& Promise)
	{
		return ToolPromises.Add(
			ToolName, MakeShared<TPromise<TValueOrError<FString, FString>>>(MoveTemp(Promise)));
	}
}

#endif