// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/Future.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "UObject/StrongObjectPtr.h"

#include "ToolsetRegistry/Toolset.h"

#if WITH_DEV_AUTOMATION_TESTS

#define UE_API TOOLSETREGISTRY_API

namespace UE::ToolsetRegistry
{

class FFakeToolset : public FToolset
{
public:
	static const inline FString SUCCESSFUL_RESULT = FString(TEXT(R"({"result": "success"})"));

	UE_API FFakeToolset(const FString& Name);
	UE_API virtual ~FFakeToolset();

	UE_API FString GetToolsetName() const override;

	UE_API FString GetToolsetVersion() const override;
	UE_API FString GetToolsetDescription() const override;

	UE_API TSharedPtr<TPromise<TValueOrError<FString, FString>>> AddFakeToolCall(
		const FString& ToolName,
		TPromise<TValueOrError<FString, FString>>&& Promise =
			TPromise<TValueOrError<FString, FString>>());

protected:
	UE_API virtual TFuture<TValueOrError<FString, FString>> ExecuteToolInternal(
		const FString& ToolName, const FString& JsonInput) override;

	UE_API virtual FString GetJsonSchemaInternal() const override;

private:
	FString Name;
	TMap<FString, TSharedPtr<TPromise<TValueOrError<FString, FString>>>> ToolPromises;
};

}

#undef UE_API

#endif
