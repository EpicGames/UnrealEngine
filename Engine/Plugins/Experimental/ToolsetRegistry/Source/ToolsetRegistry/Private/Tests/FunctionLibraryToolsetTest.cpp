// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionLibraryToolsetTest.h"

#include "CoreMinimal.h"

#include "Misc/AutomationTest.h"
#include "Dom/JsonObject.h"
#include "JsonDomBuilder.h"
#include "IPythonScriptPlugin.h"
#include "PythonScriptTypes.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Templates/SharedPointer.h"

#include "ToolsetRegistry/FunctionLibraryToolset.h"
#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/Toolset.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"
#include "ToolsetRegistry/ToolCallAsyncResultFutureHandler.h"
#include "ToolsetRegistry/UToolsetRegistry.h"
#include "Tests/ToolsetRegistryTestSchemaBuilder.h"
#include "Tests/ToolsetRegistryTestFlags.h"
#include "Tests/UntilFutureCommand.h"

#if WITH_DEV_AUTOMATION_TESTS
const FString FakeToolsetName(TEXT("ToolsetRegistry.FakeToolset"));
const FString FakeToolsetInput(TEXT(R"({"inString":"hello"})"));
const FString FakeToolName(TEXT("SimpleTool"));
const FString ReturnValueName(TEXT("returnValue"));
const FString ErrorValueName(TEXT("error"));

class ToolsetRegistryContainer
{
  public:
	ToolsetRegistryContainer(TSubclassOf<UToolsetDefinition> InToolsetClass)
		: ToolsetClass(InToolsetClass)
	{
		UToolsetRegistry::RegisterToolsetClass(ToolsetClass);
	}

	~ToolsetRegistryContainer()
	{
		UToolsetRegistry::UnregisterToolsetClass(ToolsetClass);
	}

	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> ToolsetHandler()
	{
		TSharedPtr<UE::ToolsetRegistry::FToolset> Handler;
		auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get(TEXT("Test Register"));
		if (ToolsetRegistrySubsystem.HasValue() && ToolsetClass)
		{
			FString ToolsetName =
				UE::ToolsetRegistry::FFunctionLibraryToolset::GetToolsetClassName(ToolsetClass);
			Handler = ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.Find(ToolsetName);
		}
		return StaticCastSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset>(Handler);
	}

	TSubclassOf<UToolsetDefinition> ToolsetClass;
};

TSharedPtr<FJsonObject> GetFakeToolsetExpectedSchema()
{
	FJsonDomBuilder::FObject SchemaBuilder = MakeSchemaBuilder(
		FString(FakeToolsetName), FString("1.0"), FString(TEXT("Fake Toolset that does nothing.")));

	FJsonDomBuilder::FArray ToolsArray;
	// Add SimpleTool
	{
		FJsonDomBuilder::FObject InputSchema = MakeSchema();
		FJsonDomBuilder::FObject Properties;

		FString ParamName = TEXT("inString");
		Properties.Set(ParamName, MakeProperty(TEXT("string")));
		AddPropertiesToSchema(InputSchema, Properties);
		AddRequiredListToSchema(InputSchema, TArray<FString>({ParamName}));

		FJsonDomBuilder::FObject OutputSchema = MakeSchema();
		FJsonDomBuilder::FObject OutputProperties;
		OutputProperties.Set(ReturnValueName, MakeProperty(TEXT("string")));
		AddPropertiesToSchema(OutputSchema, OutputProperties);
		AddRequiredListToSchema(OutputSchema, TArray<FString>({ReturnValueName}));

		FJsonDomBuilder::FObject ToolEntry = MakeToolSchemaEntry(
			TEXT("ToolsetRegistry.FakeToolset.SimpleTool"), TEXT("Simple tool that does nothing."),
			InputSchema, OutputSchema);

		AddToolSchema(SchemaBuilder, ToolsArray, ToolEntry);
	}

	// Add SimpleNoReturnTool
	{
		FJsonDomBuilder::FObject InputSchema = MakeSchema();
		FJsonDomBuilder::FObject Properties;

		FString ParamName = TEXT("inString");
		Properties.Set(ParamName, MakeProperty(TEXT("string")));
		AddPropertiesToSchema(InputSchema, Properties);
		AddRequiredListToSchema(InputSchema, TArray<FString>({ParamName}));

		FJsonDomBuilder::FObject ToolEntry;
		ToolEntry.Set(TEXT("description"), TEXT("Simple tool that does nothing and returns nothing."));
		ToolEntry.Set(TEXT("name"), TEXT("ToolsetRegistry.FakeToolset.SimpleNoReturnTool"));
		ToolEntry.Set(TEXT("inputSchema"), InputSchema);

		AddToolSchema(SchemaBuilder, ToolsArray, ToolEntry);
	}

	return SchemaBuilder.AsJsonObject();
}

TSharedPtr<FJsonObject> GetFakeToolsetWithDefaultParamsExpectedSchema()
{
	FJsonDomBuilder::FObject SchemaBuilder = MakeSchemaBuilder(
		FString(TEXT("ToolsetRegistry.FakeToolsetWithDefaultParams")), FString("1.0"),
		FString(TEXT("Fake Toolset that has method with default parameters.")));

	FJsonDomBuilder::FObject InputSchema = MakeSchema();
	FJsonDomBuilder::FObject Properties;

	FString ParamName1 = TEXT("inString");
	FString ParamName2 = TEXT("inValue");

	Properties.Set(ParamName1, MakeProperty(TEXT("string")));
	Properties.Set(ParamName2, MakeProperty(TEXT("integer"), MakeShared<FJsonValueNumber>(42)));
	AddPropertiesToSchema(InputSchema, Properties);
	AddRequiredListToSchema(InputSchema, TArray<FString>({ParamName1}));

	FJsonDomBuilder::FObject OutputSchema = MakeSchema();
	FJsonDomBuilder::FObject OutputProperties;
	OutputProperties.Set(ReturnValueName, MakeProperty(TEXT("string")));
	AddPropertiesToSchema(OutputSchema, OutputProperties);
	AddRequiredListToSchema(OutputSchema, TArray<FString>({ReturnValueName}));

	FJsonDomBuilder::FObject ToolEntry = MakeToolSchemaEntry(
		TEXT("ToolsetRegistry.FakeToolsetWithDefaultParams.ToolWithDefaults"),
		TEXT("Method with some params with default values."),
		InputSchema, OutputSchema);

	FJsonDomBuilder::FArray ToolsArray;
	AddToolSchema(SchemaBuilder, ToolsArray, ToolEntry);

	return SchemaBuilder.AsJsonObject();
}

TSharedPtr<FJsonObject> GetFakeToolsetWithArrayParamsExpectedSchema()
{
	FJsonDomBuilder::FObject SchemaBuilder = MakeSchemaBuilder(
		FString(TEXT("ToolsetRegistry.FakeToolsetWithArrayParams")), FString("1.0"),
		FString(TEXT("Fake Toolset that has method with TArray parameters.")));

	FJsonDomBuilder::FObject InputSchema = MakeSchema();

	FString ParamName1 = TEXT("arrayParamOne");
	FString ParamName2 = TEXT("arrayParamTwo");

	FJsonDomBuilder::FObject InputProperties;
	InputProperties.Set(ParamName1, MakeArrayProperty(TEXT("integer")));
	InputProperties.Set(ParamName2, MakeArrayProperty(TEXT("string")));
	AddPropertiesToSchema(InputSchema, InputProperties);
	AddRequiredListToSchema(InputSchema, TArray<FString>({ParamName1, ParamName2}));

	FJsonDomBuilder::FObject OutputSchema = MakeSchema();
	FJsonDomBuilder::FObject OutputProperties;
	OutputProperties.Set(ReturnValueName, MakeArrayProperty(TEXT("number")));
	AddPropertiesToSchema(OutputSchema, OutputProperties);
	AddRequiredListToSchema(OutputSchema, TArray<FString>({ReturnValueName}));

	FJsonDomBuilder::FObject ToolEntry = MakeToolSchemaEntry(
		TEXT("ToolsetRegistry.FakeToolsetWithArrayParams.ToolWithArrays"),
		TEXT("Method with some params with TArray values."),
		InputSchema, OutputSchema);

	FJsonDomBuilder::FArray ToolsArray;
	AddToolSchema(SchemaBuilder, ToolsArray, ToolEntry);

	return SchemaBuilder.AsJsonObject();
}

TSharedPtr<FJsonObject> GetFakeToolsetWithSetParamsExpectedSchema()
{
	FJsonDomBuilder::FObject SchemaBuilder = MakeSchemaBuilder(
		FString(TEXT("ToolsetRegistry.FakeToolsetWithSetParams")), FString("1.0"),
		FString(TEXT("Fake Toolset that has method with TSet parameters.")));

	FJsonDomBuilder::FObject InputSchema = MakeSchema();

	FString ParamName1 = TEXT("setParamOne");
	FString ParamName2 = TEXT("setParamTwo");

	FJsonDomBuilder::FObject InputProperties;
	InputProperties.Set(ParamName1, MakeSetProperty(TEXT("integer")));
	InputProperties.Set(ParamName2, MakeSetProperty(TEXT("string")));
	AddPropertiesToSchema(InputSchema, InputProperties);
	AddRequiredListToSchema(InputSchema, TArray<FString>({ParamName1, ParamName2}));

	FJsonDomBuilder::FObject OutputSchema = MakeSchema();
	FJsonDomBuilder::FObject OutputProperties;
	OutputProperties.Set(ReturnValueName, MakeSetProperty(TEXT("number")));
	AddPropertiesToSchema(OutputSchema, OutputProperties);
	AddRequiredListToSchema(OutputSchema, TArray<FString>({ReturnValueName}));

	FJsonDomBuilder::FObject ToolEntry = MakeToolSchemaEntry(
		TEXT("ToolsetRegistry.FakeToolsetWithSetParams.ToolWithSets"),
		TEXT("Method with some params with TSet values."),
		InputSchema, OutputSchema);

	FJsonDomBuilder::FArray ToolsArray;
	AddToolSchema(SchemaBuilder, ToolsArray, ToolEntry);

	return SchemaBuilder.AsJsonObject();
}

TSharedPtr<FJsonObject> GetFakeToolsetWithMapParamsExpectedSchema()
{
	FJsonDomBuilder::FObject SchemaBuilder = MakeSchemaBuilder(
		FString(TEXT("ToolsetRegistry.FakeToolsetWithMapParams")), FString("1.0"),
		FString(TEXT("Fake Toolset that has method with TMap parameters.")));

	FJsonDomBuilder::FObject InputSchema = MakeSchema();

	FString ParamName1 = TEXT("mapParamOne");
	FString ParamName2 = TEXT("mapParamTwo");

	FJsonDomBuilder::FObject InputProperties;
	InputProperties.Set(ParamName1, MakeMapProperty(TEXT("integer")));
	InputProperties.Set(ParamName2, MakeMapProperty(TEXT("string")));
	AddPropertiesToSchema(InputSchema, InputProperties);
	AddRequiredListToSchema(InputSchema, TArray<FString>({ParamName1, ParamName2}));

	FJsonDomBuilder::FObject OutputSchema = MakeSchema();
	FJsonDomBuilder::FObject OutputProperties;
	OutputProperties.Set(ReturnValueName, MakeMapProperty(TEXT("number")));
	AddPropertiesToSchema(OutputSchema, OutputProperties);
	AddRequiredListToSchema(OutputSchema, TArray<FString>({ReturnValueName}));

	FJsonDomBuilder::FObject ToolEntry = MakeToolSchemaEntry(
		TEXT("ToolsetRegistry.FakeToolsetWithMapParams.ToolWithMaps"),
		TEXT("Method with some params with TMap values."),
		InputSchema, OutputSchema);

	FJsonDomBuilder::FArray ToolsArray;
	AddToolSchema(SchemaBuilder, ToolsArray, ToolEntry);

	return SchemaBuilder.AsJsonObject();
}

// Execute a tool in the context of a test case.
void UToolsetRegistryExecuteTool(
	FAutomationTestBase& Test, const FString& ToolsetName, const FString& ToolName,
	const FString& JsonInput, TFunction<void(UE::ToolsetRegistry::FJsonValueOrError&&)>&& OnResult)
{
	Test.AddCommand(
		UE::ToolsetRegistry::Internal::FUntilFutureCommand::Create<
			UE::ToolsetRegistry::FJsonValueOrError>(
				UToolCallAsyncResultFutureHandler::Create(
					UToolsetRegistry::ExecuteTool(
						ToolsetName, ToolName, JsonInput))->GetValueAsJson(),
				MoveTemp(OnResult)));
}

// Test that registering with null toolset class input fails properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestRegisterNullClass,
	"AI.ToolsetRegistry.FunctionLibraryToolset.RegisterNullClass",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestRegisterNullClass::RunTest(const FString& UnusedParameters)
{
	AddExpectedError(
		TEXT("Cannot register null toolset class"), EAutomationExpectedErrorFlags::Contains);
	ToolsetRegistryContainer Container(nullptr);
	TestTrue("Null toolset class", Container.ToolsetClass == nullptr);

	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset =
		Container.ToolsetHandler();
	TestTrue("Null toolset wrapper", Toolset == nullptr);

	return true;
}

// Test the function library toolset with null toolset class input fails properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestConstructWithNullClass,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ConstructWithNullClass",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestConstructWithNullClass::RunTest(
	const FString& UnusedParameters)
{
	UE::ToolsetRegistry::FFunctionLibraryToolset Toolset(nullptr);
	TestFalse("Invalid function library toolset with null class", Toolset.HasValidTools());
	return true;
}

// Test that toolset can unregister and re-register itself properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestRegisterUnregisterToolset,
	"AI.ToolsetRegistry.FunctionLibraryToolset.RegisterUnregisterToolset",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestRegisterUnregisterToolset::RunTest(const FString& UnusedParameters)
{
	UClass* ToolsetClass = UFakeToolset::StaticClass();

	TestFalse("Toolset unregistered before registering itself.",
			  UToolsetRegistry::IsToolsetClassRegistered(ToolsetClass));

	// Scope for registration of ToolsetClass.
	{
		ToolsetRegistryContainer Container(ToolsetClass);
		TestTrue("Toolset registered after registering itself.",
				 UToolsetRegistry::IsToolsetClassRegistered(ToolsetClass));

		UToolsetRegistry::UnregisterToolsetClass(ToolsetClass);
		TestFalse("Toolset unregistered after unregistering itself.",
				  UToolsetRegistry::IsToolsetClassRegistered(ToolsetClass));

		UToolsetRegistry::RegisterToolsetClass(ToolsetClass);
		TestTrue("Toolset registered after re-registering itself.",
				 UToolsetRegistry::IsToolsetClassRegistered(ToolsetClass));
	}
	return true;
}

// Test that toolset with no tools fails to register.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestNoToolsRegistrationRejected,
	"AI.ToolsetRegistry.FunctionLibraryToolset.NoToolsRegistrationRejected",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestNoToolsRegistrationRejected::RunTest(
	const FString& UnusedParameters)
{
	UClass* ToolsetClass = UFakeToolsetWithNoTools::StaticClass();
	AddExpectedError(TEXT("invalid tool list"));
	ToolsetRegistryContainer Container(ToolsetClass);
	TestFalse("Toolset with no tools is not registered.",
			  UToolsetRegistry::IsToolsetClassRegistered(ToolsetClass));
	return true;
}

// Test that toolset can return its name, version, and description.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestCanGetValues,
	"AI.ToolsetRegistry.FunctionLibraryToolset.CanGetValues",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestCanGetValues::RunTest(const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolset::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue("Valid toolset handler returned", Toolset.IsValid())) return false;

	TestEqual("Toolset returns its own name",
			  Toolset->GetToolsetName(), FakeToolsetName);
	TestEqual("Toolset returns default non-overridden version",
			  Toolset->GetToolsetVersion(), FString(TEXT("1.0")));
	TestEqual("Toolset returns description from class definition",
			  Toolset->GetToolsetDescription(), FString(TEXT("Fake Toolset that does nothing.")));
	return true;
}

// Test that toolset can return a basic schema properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestGetSchema,
	"AI.ToolsetRegistry.FunctionLibraryToolset.GetSchema",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestGetSchema::RunTest(const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolset::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue("Valid toolset handler returned", Toolset.IsValid())) return false;

	TSharedPtr<FJsonObject> ExpectedSchema = GetFakeToolsetExpectedSchema();

	FString ExpectedSchemaJson =
		UE::ToolsetRegistry::Internal::JsonToString(ExpectedSchema.ToSharedRef());
	FString OutputSchemaJson = Toolset->GetJsonSchema();
	TestEqual("Toolset schema matches expected", OutputSchemaJson, ExpectedSchemaJson);
	return true;
}

// Test that toolset with method with default values can return a schema properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestGetSchemaWithDefaultParams,
	"AI.ToolsetRegistry.FunctionLibraryToolset.GetSchemaWithDefaultParams",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestGetSchemaWithDefaultParams::RunTest(
	const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolsetWithDefaultParams::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue("Valid toolset handler returned", Toolset.IsValid())) return true;

	TSharedPtr<FJsonObject> ExpectedSchema = GetFakeToolsetWithDefaultParamsExpectedSchema();

	FString ExpectedSchemaJson =
		UE::ToolsetRegistry::Internal::JsonToString(ExpectedSchema.ToSharedRef());
	FString OutputSchemaJson = Toolset->GetJsonSchema();
	TestEqual("Toolset schema matches expected", OutputSchemaJson, ExpectedSchemaJson);
	return true;
}

// Test that toolset with method with array params can return a schema properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestGetSchemaWithArrayParams,
	"AI.ToolsetRegistry.FunctionLibraryToolset.GetSchemaWithArrayParams",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestGetSchemaWithArrayParams::RunTest(const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolsetWithArrayParams::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue("Valid toolset handler returned", Toolset.IsValid())) return true;

	TSharedPtr<FJsonObject> ExpectedSchema = GetFakeToolsetWithArrayParamsExpectedSchema();

	FString ExpectedSchemaJson =
		UE::ToolsetRegistry::Internal::JsonToString(ExpectedSchema.ToSharedRef());
	FString OutputSchemaJson = Toolset->GetJsonSchema();
	TestEqual("Toolset schema matches expected", OutputSchemaJson, ExpectedSchemaJson);
	return true;
}

// Test that toolset with method with map params can return a schema properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestGetSchemaWithMapParams,
	"AI.ToolsetRegistry.FunctionLibraryToolset.GetSchemaWithMapParams",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestGetSchemaWithMapParams::RunTest(const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolsetWithMapParams::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue("Valid toolset handler returned", Toolset.IsValid())) return true;

	TSharedPtr<FJsonObject> ExpectedSchema = GetFakeToolsetWithMapParamsExpectedSchema();

	FString ExpectedSchemaJson =
		UE::ToolsetRegistry::Internal::JsonToString(ExpectedSchema.ToSharedRef());
	FString OutputSchemaJson = Toolset->GetJsonSchema();
	TestEqual("Toolset schema matches expected", OutputSchemaJson, ExpectedSchemaJson);
	return true;
}

// Test that toolset with method with set params can return a schema properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestGetSchemaWithSetParams,
	"AI.ToolsetRegistry.FunctionLibraryToolset.GetSchemaWithSetParams",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestGetSchemaWithSetParams::RunTest(const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolsetWithSetParams::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue("Valid toolset handler returned", Toolset.IsValid())) return true;

	TSharedPtr<FJsonObject> ExpectedSchema = GetFakeToolsetWithSetParamsExpectedSchema();

	FString ExpectedSchemaJson =
		UE::ToolsetRegistry::Internal::JsonToString(ExpectedSchema.ToSharedRef());
	FString OutputSchemaJson = Toolset->GetJsonSchema();
	TestEqual("Toolset schema matches expected", OutputSchemaJson, ExpectedSchemaJson);
	return true;
}

// Test that invoking a tool from a toolset calls the right method and returns the expected result.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecuteTool,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecuteTool",
	ToolsetRegistryTest::Flags)

	bool FFunctionLibraryToolsetTestExecuteTool::RunTest(const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolset::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue(TEXT("Valid toolset handler returned"), Toolset.IsValid())) return false;

	AddCommand(
		UE::ToolsetRegistry::Internal::FUntilFutureCommand::Create<
				TValueOrError<FString, FString>>(
			Toolset->ExecuteTool(FakeToolName, FakeToolsetInput),
			[this](TValueOrError<FString, FString>&& Result) -> void
			{
				if (TestTrue(TEXT("Tool has value"), Result.HasValue()))
				{
					TestEqual(TEXT("Tool execution returns expected output"),
						Result.GetValue(),
						TEXT(R"({"returnValue":"hello"})"));
				}
			}));
	return true;
}

// Test that invoking a tool from an invalid named toolset errors out properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecuteToolInvalidToolsetName,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecuteToolInvalidToolsetName",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestExecuteToolInvalidToolsetName::RunTest(
	const FString& UnusedParameters)
{
	AddExpectedError(TEXT("not found"), EAutomationExpectedErrorFlags::Contains);
	UToolsetRegistryExecuteTool(
		*this, TEXT("ToolsetRegistry.FakeToolsetInvalid"), FakeToolName, FakeToolsetInput,
		[this](UE::ToolsetRegistry::FJsonValueOrError&& Result) -> void
		{
			TestTrue(
				TEXT("Tool execution with invalid toolset name returns error"),
				Result.HasError());
		});
	return true;
}

// Test that invoking a tool with an invalid tool name errors out properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecuteToolInvalidToolname,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecuteToolInvalidToolname",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestExecuteToolInvalidToolname::RunTest(
	const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolset::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	TestTrue("Valid toolset handler returned", Toolset.IsValid());

	UToolsetRegistryExecuteTool(
		*this, FakeToolsetName, TEXT("SimpleToolInvalid"), FakeToolsetInput,
		[this](UE::ToolsetRegistry::FJsonValueOrError&& Result) -> void
		{
			TestTrue(
				TEXT("Tool execution with invalid tool name returns error"),
				Result.HasError());
		});
	return true;
}

// Test that invoking a tool with a valid but unregistered toolset errors out properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecuteToolUnregisteredToolset,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecuteToolUnregisteredToolset",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestExecuteToolUnregisteredToolset::RunTest(
	const FString& UnusedParameters)
{
	// Ensure it is unregistered
	UToolsetRegistry::UnregisterToolsetClass(UFakeToolset::StaticClass());

	AddExpectedError(TEXT("not found"), EAutomationExpectedErrorFlags::Contains);
	UToolsetRegistryExecuteTool(
		*this, FakeToolsetName, FakeToolName, FakeToolsetInput,
		[this](UE::ToolsetRegistry::FJsonValueOrError&& Result) -> void
		{
			TestTrue(
				TEXT("Tool execution with unregistered toolset returns error"),
				Result.HasError());
		});
	return true;
}

// Test that invoking a tool that doesn't return anything doesn't raise errors.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecuteToolWithNoReturn,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecuteToolWithNoReturn",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestExecuteToolWithNoReturn::RunTest(
	const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolset::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	TestTrue("Valid toolset handler returned", Toolset.IsValid());

	UToolsetRegistryExecuteTool(
		*this, FakeToolsetName, TEXT("SimpleNoReturnTool"), FakeToolsetInput,
		[this](UE::ToolsetRegistry::FJsonValueOrError&& Result) -> void
		{
			if (!TestFalse(
				TEXT("Tool execution with no return value runs without error"),
				Result.HasError()))
			{
				return;
			}
			TestEqual(
				TEXT("Tool execution returns expected output"),
				Result.GetValue()->AsString(),
				TEXT(R"({"returnValue":null})"));
		});
	return true;
}

// Test that invoking a tool with default values for params provides the proper default value.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecuteToolWithDefaultParam,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecuteToolWithDefaultParam",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestExecuteToolWithDefaultParam::RunTest(
	const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolsetWithDefaultParams::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	TestTrue("Valid toolset handler returned", Toolset.IsValid());

	UToolsetRegistryExecuteTool(
		*this,
		TEXT("ToolsetRegistry.FakeToolsetWithDefaultParams"),
		TEXT("ToolWithDefaults"),
		FakeToolsetInput,
		[this](UE::ToolsetRegistry::FJsonValueOrError&& Result) -> void
		{
			if (!TestFalse(
					TEXT("Tool execution with default param returns without error"),
					Result.HasError()))
			{
				return;
			}
			TestEqual(
				TEXT("Tool execution returns expected output"),
				Result.GetValue()->AsString(),
				TEXT(R"({"returnValue":"hello: 42"})"));
		});
	return true;
}

// Test that invoking a tool with params that don't match input schema errors out properly.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecuteToolWithBadInput,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecuteToolWithBadInput",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestExecuteToolWithBadInput::RunTest(
	const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolset::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	TestTrue(TEXT("Valid toolset handler returned"), Toolset.IsValid());

	FString Input = TEXT(R"({"not_expected":"bad_value"})");

	UToolsetRegistryExecuteTool(
		*this,
		FakeToolsetName, FakeToolName, Input,
		[this](UE::ToolsetRegistry::FJsonValueOrError&& Result) -> void
		{
			TestTrue(
				TEXT("Tool execution with invalid param returns with error"),
				Result.HasError());
		});
	return true;
}

// Test that a toolset doesn't include non-AICallable non-static methods.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestValidateToolMethods,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ValidateToolMethods",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestValidateToolMethods::RunTest(
	const FString& UnusedParameters)
{
	AddExpectedError(TEXT("is not static and will not be added as a Tool."), 
		EAutomationExpectedMessageFlags::Contains);
	AddExpectedError(TEXT("is not AICallable and will not be added as a Tool."), 
		EAutomationExpectedMessageFlags::Contains);
	ToolsetRegistryContainer Container(UFakeToolsetWithInvalidUFunctions::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue("Valid toolset handler returned", Toolset.IsValid()))
	{
		return false;
	}

	TArray<FString> ToolNames = Toolset->GetToolNames();

	if (!TestEqual("Only one valid tool should be included", ToolNames.Num(), 1))
	{
		return false;
	}
	TestEqual("Only \"SimplePrivateTool\" should be valid tool", ToolNames[0], FString("SimplePrivateTool"));

	return true;
}

// Invoke a C++ tool that produces errors, not from bad input.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecuteCppToolWithError,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecuteCppToolWithError",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestExecuteCppToolWithError::RunTest(
	const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeErrorProneToolset::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	TestTrue(TEXT("Valid toolset handler returned"), Toolset.IsValid());

	UToolsetRegistryExecuteTool(
		*this,
		Toolset->GetToolsetName(), TEXT("BuggyTool"), TEXT(""),
		[this](UE::ToolsetRegistry::FJsonValueOrError&& Result) -> void
		{
			if (!TestTrue(
					TEXT("Tool execution with invalid param returns with error"),
					Result.HasError()))
			{
				return;
			}
			TestEqual(
				TEXT("Tool execution with buggy tool returns correct raised exception"),
				Result.GetError(), UFakeErrorProneToolset::ErrorMessage);
		});
	return true;
}

// Invoke a C++ tool that produces errors from invalid object param checks.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecuteCppToolWithBadParam,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecuteCppToolWithBadParam",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestExecuteCppToolWithBadParam::RunTest(
	const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolsetWithObjectParams::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue("Valid toolset handler returned", Toolset.IsValid()))
	{
		return false;
	}

	UToolsetRegistryExecuteTool(
		*this,
		Toolset->GetToolsetName(),
		TEXT("TestObjectParam"),
		TEXT(R"({"testObject": {"refPath": "/Invalid/Path/DoesNotExist"}})"),
		[this](UE::ToolsetRegistry::FJsonValueOrError&& Result) -> void
		{
			if (!TestTrue(
					TEXT("Tool call with invalid object param has error"),
					Result.HasError()))
			{
				return;
			}
			TestTrue(
				TEXT("Tool call with invalid object param error"),
				Result.GetError().Contains(TEXT("is not a valid")));
		});
	return true;
}

// Invoke python tool that produces errors.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestExecutePythonToolWithError,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ExecutePythonToolWithError",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestExecutePythonToolWithError::RunTest(
	const FString& UnusedParameters)
{
	if (!IPythonScriptPlugin::Get()->IsPythonAvailable())
	{
		return true;
	}

	FPythonCommandEx PythonCommand = FPythonCommandEx();
	PythonCommand.ExecutionMode = EPythonCommandExecutionMode::ExecuteStatement;
	PythonCommand.Command = TEXT(
		"from toolset_registry.tests.demo_toolset import ErrorProneToolset; "
		"import unreal; unreal.ToolsetRegistry.register_toolset_class(ErrorProneToolset)");
	if (!TestTrue(TEXT("Register ErrorProneToolset"),
			IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand)))
	{
		return false;
	}

	UToolsetRegistryExecuteTool(
		*this,
		TEXT("toolset_registry.tests.demo_toolset.ErrorProneToolset"), TEXT("bad_tool"),
		TEXT(R"({"msg":"Hi"})"),
		[this](UE::ToolsetRegistry::FJsonValueOrError&& Result) -> void
		{
			if (TestTrue(TEXT("Tool error was returned"), Result.HasError()))
			{
				TestEqual(
					TEXT("Error is expected exception message"), Result.GetError(),
					TEXT("Bad tool bad"));
			}
			FPythonCommandEx PythonCommand = FPythonCommandEx();
			PythonCommand.ExecutionMode = EPythonCommandExecutionMode::ExecuteStatement;
			PythonCommand.Command = TEXT(
				"from toolset_registry.tests.demo_toolset import ErrorProneToolset; "
				"import unreal; unreal.ToolsetRegistry.unregister_toolset_class(ErrorProneToolset)");
			TestTrue(
				"Unregister ErrorProneToolset",
				IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand));
		});
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestToolsetNaming,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ToolsetNaming",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestToolsetNaming::RunTest(
	const FString& UnusedParameters)
{
	UClass* ToolsetClass = UFakeToolset::StaticClass();
	FString ToolsetName = UE::ToolsetRegistry::FFunctionLibraryToolset::GetToolsetClassName(ToolsetClass);
	FString PackageName = ToolsetClass->GetOuter()->GetName();
	FString PathName = ToolsetClass->GetPathName();
	FString ClassName = ToolsetClass->GetName();
	TestNotEqual("Toolset name should not be Class name", ToolsetName, ClassName);
	TestNotEqual("Toolset name should not be same as Path name", ToolsetName, PathName);
	TestTrue("Toolset name should include Class name", ToolsetName.Contains(ClassName));
	int32 DotIndex;
	TestTrue("Toolset name should include a namespace qualifier", ToolsetName.FindLastChar('.', DotIndex));
	FString ToolsetNamespace = ToolsetName.Left(DotIndex);
	FString ToolsetNameClass = ToolsetName.RightChop(DotIndex + 1);
	TestEqual("Toolset name should have class name last", ToolsetNameClass, ClassName);
	int32 SlashIndex;
	TestTrue("Toolset package should be slash-separated", PackageName.FindLastChar('/', SlashIndex));
	FString PackageLast = PackageName.RightChop(SlashIndex + 1);
	TestEqual("Toolset name namespace should be rightmost package part", ToolsetNamespace, PackageLast);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestToolsetVersionOverride,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ToolsetVersionOverride",
	ToolsetRegistryTest::Flags)

bool FFunctionLibraryToolsetTestToolsetVersionOverride::RunTest(
	const FString& UnusedParameters)
{
	ToolsetRegistryContainer Container(UFakeToolsetWithVersionOverride::StaticClass());
	TSharedPtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset = Container.ToolsetHandler();
	if (!TestTrue("Valid toolset handler returned", Toolset.IsValid())) return false;

	TestEqual("Toolset returns overridden version",
		Toolset->GetToolsetVersion(), FString(TEXT("1.5")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FFunctionLibraryToolsetTestToolsetNamingNull,
	"AI.ToolsetRegistry.FunctionLibraryToolset.ToolsetNamingNull",
	ToolsetRegistryTest::Flags)

	bool FFunctionLibraryToolsetTestToolsetNamingNull::RunTest(
		const FString& UnusedParameters)
{
	FString ToolsetName = UE::ToolsetRegistry::FFunctionLibraryToolset::GetToolsetClassName(nullptr);
	return TestTrue("Toolset name for null UClass should be empty", ToolsetName.IsEmpty());
}

DEFINE_SPEC(FToolsetRegistryToolsetTestGetAllSchemas,
			"AI.ToolsetRegistry.FunctionLibraryToolset.GetAllSchemas", ToolsetRegistryTest::Flags)

void FToolsetRegistryToolsetTestGetAllSchemas::Define()
{
	Describe("UToolsetRegistry::GetAllToolsetJsonSchemas", [this]()
	{
		It("should return exactly what FToolsetRegistry::GetToolsetJsonSchemas does", [this]()
		{
			auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get();
			if (!ToolsetRegistrySubsystem.HasValue())
			{
				AddError(TEXT("Toolset Registry subsystem unavailable"));
				return;
			}
			FString AllSchemas =
				ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.GetToolsetJsonSchemas();
			TestEqual(TEXT("Schemas text output"), AllSchemas,
					  UToolsetRegistry::GetAllToolsetJsonSchemas());
		});
	});
}

#endif	// WITH_DEV_AUTOMATION_TESTS

