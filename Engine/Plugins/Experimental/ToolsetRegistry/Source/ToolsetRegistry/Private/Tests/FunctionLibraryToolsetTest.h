// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Dom/JsonObject.h"
#include "Kismet/KismetSystemLibrary.h"

#include "ToolsetRegistry/ToolsetDefinition.h"

#include "FunctionLibraryToolsetTest.generated.h"

/** Fake Toolset that does nothing. */
UCLASS(Blueprintable, Hidden)
class UFakeToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/** Simple tool that does nothing. */
	UFUNCTION(meta = (AICallable), Category=Fake)
	static FString SimpleTool(const FString& InString)
	{
		return InString;
	}

	/** Simple tool that does nothing and returns nothing. */
	UFUNCTION(meta = (AICallable), Category=Fake)
	static void SimpleNoReturnTool(const FString& InString)
	{
		return;
	}

	/** UFunction marked as AIIgnore, should not be marked as tool. */
	UFUNCTION(meta = (AIIgnore))
	static void SimpleInternalMethod() { }
};

static TSharedPtr<FJsonObject> GetFakeToolsetExpectedSchema();

/** Fake Toolset that has method with default parameters. */
UCLASS(Blueprintable, Hidden)
class UFakeToolsetWithDefaultParams : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/** Method with some params with default values. */
	UFUNCTION(meta = (AICallable), Category=Fake)
	static FString ToolWithDefaults(const FString& InString, int InValue=42)
	{
		return FString::Printf(TEXT("%s: %d"), *InString, InValue);
	}
};

static TSharedPtr<FJsonObject> GetFakeToolsetWithDefaultParamsExpectedSchema();


/** Fake Toolset with no tools. */
UCLASS(Blueprintable, Hidden)
class UFakeToolsetWithNoTools : public UToolsetDefinition
{
	GENERATED_BODY()
};

static TSharedPtr<FJsonObject> GetFakeToolsetWithNoToolsExpectedSchema();

/** Fake Toolset that has method with TArray parameters. */
UCLASS(Blueprintable, Hidden)
class UFakeToolsetWithArrayParams : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/** Method with some params with TArray values. */
	UFUNCTION(meta = (AICallable), Category=Fake)
	static TArray<float> ToolWithArrays(const TArray<int> ArrayParamOne, TArray<FString> ArrayParamTwo)
	{
		return TArray<float>();
	}
};

static TSharedPtr<FJsonObject> GetFakeToolsetWithArrayParamsExpectedSchema();

/** Fake Toolset that has method with TMap parameters. */
UCLASS(Blueprintable, Hidden)
class UFakeToolsetWithMapParams : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/** Method with some params with TMap values. */
	UFUNCTION(meta = (AICallable), Category=Fake)
	static TMap<FString, float> ToolWithMaps(const TMap<FString, int> MapParamOne, TMap<FString, FString> MapParamTwo)
	{
		return TMap<FString, float>();
	}
};

static TSharedPtr<FJsonObject> GetFakeToolsetWithMapParamsExpectedSchema();

/** Fake Toolset that has method with TSet parameters. */
UCLASS(Blueprintable, Hidden)
class UFakeToolsetWithSetParams : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/** Method with some params with TSet values. */
	UFUNCTION(meta = (AICallable), Category=Fake)
	static TSet<float> ToolWithSets(const TSet<int> SetParamOne, TSet<FString> SetParamTwo)
	{
		return TSet<float>();
	}
};

static TSharedPtr<FJsonObject> GetFakeToolsetWithSetParamsExpectedSchema();

/** Fake Toolset with invalid UFunctions. */
UCLASS(Blueprintable, Hidden)
class UFakeToolsetWithInvalidUFunctions : public UToolsetDefinition
{
	GENERATED_BODY()

	/** Untagged ufunction */
	UFUNCTION()
	static void UntaggedFunction() {}

	/** Non-static function */
	UFUNCTION(meta = (AICallable))
	void NonStaticFunction() const {}

	/** AICallable UFunction private method, should be marked as tool. */
	UFUNCTION(meta = (AICallable))
	static void SimplePrivateTool() {}
};


/** Fake Toolset that has error-prone method. */
UCLASS(Blueprintable, Hidden)
class UFakeErrorProneToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	static inline FString ErrorMessage = TEXT("Bug in this tool");

	/** Method that raises errors. */
	UFUNCTION(meta = (AICallable), Category=Fake)
	static void BuggyTool()
	{
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
	}
};

/** Fake Toolset that with methods taking object and class params. */
UCLASS(Blueprintable, Hidden)
class UFakeToolsetWithObjectParams : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	UFUNCTION(meta = (AICallable), Category = Fake)
	static void TestObjectParam(UObject* TestObject) {}

	UFUNCTION(meta = (AICallable), Category = Fake)
	static void TestClassParam(UClass* TestClass) {}
};

/** Fake Toolset with non-default version. */
UCLASS(Blueprintable, Hidden)
class UFakeToolsetWithVersionOverride : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/** AICallable  method, should be marked as tool. */
	UFUNCTION(meta = (AICallable))
	static void SimpleTool() {}

	virtual FString GetToolsetVersion() const override
	{
		return FString(TEXT("1.5"));
	}
};
