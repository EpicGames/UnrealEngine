// Copyright Epic Games, Inc. All Rights Reserved.

#include "GuidConverterTest.h"

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/AutomationTest.h"
#include "Misc/Guid.h"

#include "ToolsetRegistry/JsonSchema.h"
#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/ToolsetJson.h"
#include "Tests/ToolCallTestHelpers.h"
#include "UObject/Class.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	const FGuid SampleGuid(0xE05FCC13, 0x4D379D7A, 0xE2388385, 0x9F29AD74);
	const FString SampleHyphenated(TEXT("E05FCC13-4D37-9D7A-E238-83859F29AD74"));
	const FString ZeroHyphenated(TEXT("00000000-0000-0000-0000-000000000000"));
}

BEGIN_DEFINE_SPEC(FToolsetGuidConverterSpec, "AI.ToolsetRegistry.GuidConverterSpec",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolsetGuidConverterSpec)

void FToolsetGuidConverterSpec::Define()
{
	using namespace UE::ToolsetRegistry::Internal;
	using namespace UE::ToolsetRegistry::Internal::ToolsetJson;

	It("Publishes a string-typed schema for FGuid properties", [this]()
	{
		UStruct* Struct = FGuidConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestGuid");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Property);
		if (!TestTrue("Schema", Schema.IsValid()))
		{
			return;
		}

		FString Type;
		TestTrue("Type", Schema->TryGetStringField(TEXT("type"), Type));
		TestEqual("Type", Type, FString(TEXT("string")));

		TestFalse("NoProperties", Schema->HasField(TEXT("properties")));

		FString Description;
		TestTrue("Description", Schema->TryGetStringField(TEXT("description"), Description));
		TestFalse("DescriptionNonEmpty", Description.IsEmpty());

		FString Pattern;
		TestTrue("Pattern", Schema->TryGetStringField(TEXT("pattern"), Pattern));
		TestFalse("PatternNonEmpty", Pattern.IsEmpty());
	});

	It("Serializes FGuid as a hyphenated string", [this]()
	{
		UStruct* Struct = FGuidConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestGuid");

		FGuidConverterTest TestIn;
		TestIn.TestGuid = SampleGuid;

		const TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestGuid);
		if (!TestTrue("OutJson", OutJson.IsValid()))
		{
			return;
		}

		TestEqual("Type", static_cast<int32>(OutJson->Type), static_cast<int32>(EJson::String));
		TestEqual("Value", OutJson->AsString(), SampleHyphenated);
	});

	It("Round-trips FGuid through JSON", [this]()
	{
		UStruct* Struct = FGuidConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestGuid");

		FGuidConverterTest TestIn;
		TestIn.TestGuid = SampleGuid;

		const TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestGuid);

		FGuidConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestGuid));
		TestEqual("GuidCheck", TestOut.TestGuid, TestIn.TestGuid);
	});

	It("Rejects the un-hyphenated form", [this]()
	{
		TObjectPtr<UGuidConverterTestObject> TestObject = NewObject<UGuidConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result = UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
			TestObject, TEXT("TestGuidParam"),
			TEXT(R"({"testGuid": "E05FCC134D379D7AE23883859F29AD74"})"));

		TestTrue("Should have error", Result.HasError());
	});

	It("Rejects the brace-wrapped form", [this]()
	{
		TObjectPtr<UGuidConverterTestObject> TestObject = NewObject<UGuidConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result = UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
			TestObject, TEXT("TestGuidParam"),
			FString::Printf(TEXT(R"({"testGuid": "{%s}"})"), *SampleHyphenated));

		TestTrue("Should have error", Result.HasError());
	});

	It("Rejects non-string JSON input", [this]()
	{
		TObjectPtr<UGuidConverterTestObject> TestObject = NewObject<UGuidConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result = UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
			TestObject, TEXT("TestGuidParam"),
			TEXT(R"({"testGuid": 42})"));

		TestTrue("Should have error", Result.HasError());
	});

	It("Rejects malformed string input", [this]()
	{
		TObjectPtr<UGuidConverterTestObject> TestObject = NewObject<UGuidConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result = UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
			TestObject, TEXT("TestGuidParam"),
			TEXT(R"({"testGuid": "not-a-guid"})"));

		TestTrue("Should have error", Result.HasError());
	});

	It("Accepts canonical hyphenated GUID via tool call", [this]()
	{
		TObjectPtr<UGuidConverterTestObject> TestObject = NewObject<UGuidConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result = UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
			TestObject, TEXT("TestGuidParam"),
			FString::Printf(TEXT(R"({"testGuid": "%s"})"), *SampleHyphenated));

		TestFalse("Should not have error", Result.HasError());
	});

	It("Accepts lowercase hyphenated GUID via tool call", [this]()
	{
		TObjectPtr<UGuidConverterTestObject> TestObject = NewObject<UGuidConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result = UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
			TestObject, TEXT("TestGuidParam"),
			FString::Printf(TEXT(R"({"testGuid": "%s"})"), *SampleHyphenated.ToLower()));

		TestFalse("Should not have error", Result.HasError());
	});

	It("Rejects 36-char hyphenated input with non-hex characters", [this]()
	{
		TObjectPtr<UGuidConverterTestObject> TestObject = NewObject<UGuidConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result = UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
			TestObject, TEXT("TestGuidParam"),
			TEXT(R"({"testGuid": "XXXXXXXX-XXXX-XXXX-XXXX-XXXXXXXXXXXX"})"));

		TestTrue("Should have error", Result.HasError());
	});

	It("Can read FGuid default in UFunction inputSchema", [this]()
	{
		UFunction* FunctionPtr = UGuidConverterTestObject::StaticClass()->FindFunctionByName("TestDefault");
		const TSharedPtr<FJsonObject> Schema = StructToJsonSchema(FunctionPtr);
		if (!TestTrue("Schema", Schema.IsValid()))
		{
			return;
		}

		const TSharedPtr<FJsonObject>* InputSchema = nullptr;
		if (!(TestTrue("Input", Schema->TryGetObjectField(TEXT("inputSchema"), InputSchema)) && InputSchema))
		{
			return;
		}

		const TSharedPtr<FJsonObject>* Properties = nullptr;
		if (!(TestTrue("Properties", InputSchema->Get()->TryGetObjectField(TEXT("properties"), Properties)) && Properties))
		{
			return;
		}

		const TSharedPtr<FJsonObject>* TestGuidSchema = nullptr;
		if (!(TestTrue("testGuid", Properties->Get()->TryGetObjectField(TEXT("testGuid"), TestGuidSchema)) && TestGuidSchema))
		{
			return;
		}

		FString Type;
		TestTrue("Type", TestGuidSchema->Get()->TryGetStringField(TEXT("type"), Type));
		TestEqual("Type", Type, FString(TEXT("string")));

		FString Default;
		TestTrue("Default", TestGuidSchema->Get()->TryGetStringField(TEXT("default"), Default));
		TestEqual("Default", Default, ZeroHyphenated);
	});
}

#endif
