// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GuidConverterTest.generated.h"


UCLASS(BlueprintType)
class UGuidConverterTestObject : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category="GuidConverterTest")
	void TestGuidParam(FGuid TestGuid) {}

	UFUNCTION(BlueprintCallable, Category="GuidConverterTest")
	void TestDefault(FGuid TestGuid = FGuid()) {}
};

USTRUCT(BlueprintType)
struct FGuidConverterTest
{
	GENERATED_BODY()
public:
	UPROPERTY()
	FGuid TestGuid;
};
