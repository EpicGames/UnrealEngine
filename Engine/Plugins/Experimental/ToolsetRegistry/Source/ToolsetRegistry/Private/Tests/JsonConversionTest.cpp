// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"

#include "ToolsetRegistry/JsonConversion.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(
	FToolsetRegistryJsonConversionSpec,
	"AI.ToolsetRegistry.JsonConversionTest",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolsetRegistryJsonConversionSpec)

void FToolsetRegistryJsonConversionSpec::Define()
{
	using namespace UE::ToolsetRegistry::Internal;

	Describe(TEXT("JsonStringToJsonObject"), [this]()
	{
		It(TEXT("should return no object from an empty string"), [this]()
		{
			TSharedPtr<FJsonObject> JsonObject = JsonStringToJsonObject(TEXT(""));
			TestFalse(TEXT("IsValid"), JsonObject.IsValid());
		});

		It(TEXT("should return no object given an invalid string"), [this]()
		{
			TSharedPtr<FJsonObject> JsonObject =
				JsonStringToJsonObject(TEXT(R"json("a_value")json"));
			TestFalse(TEXT("IsValid"), JsonObject.IsValid());
		});

		It(TEXT("should return an object from a valid object string"), [this]()
		{
			TSharedPtr<FJsonObject> JsonObject =
				JsonStringToJsonObject(TEXT(R"json({"hello": "world"})json"));
			if (!TestTrue(TEXT("IsValid"), JsonObject.IsValid())) return;
			
			TSharedPtr<FJsonValue> JsonValue = JsonObject->TryGetField(TEXT("hello"));
			if (!TestTrue(TEXT("Has 'hello' field"), JsonValue.IsValid())) return;

			TestEqual(TEXT("Field value"), JsonValue->AsString(), TEXT("world"));
		});
	});

	Describe(TEXT("JsonObjectOrEmpty"), [this]()
	{
		It(TEXT("Should return an existing object when provided an object"), [this]()
		{
			TSharedPtr<FJsonObject> JsonObject =
				JsonStringToJsonObject(TEXT(R"json({"hello": "world"})json"));
			TestEqual(TEXT("Json object"), &JsonObjectOrEmpty(JsonObject).Get(), JsonObject.Get());
		});

		It(TEXT("Should return an empty when provided no object"), [this]()
		{
			TSharedRef<FJsonObject> JsonObject = JsonObjectOrEmpty(TSharedPtr<FJsonObject>());
			TestTrue(TEXT("Json object with no values"), JsonObject->Values.IsEmpty());
		});
	});

	Describe(TEXT("JsonStringToJsonValue"), [this]()
	{
		It(TEXT("should return no value from an empty string"), [this]()
		{
			TSharedPtr<FJsonValue> JsonValue = JsonStringToJsonValue(TEXT(""));
			TestFalse(TEXT("IsValid"), JsonValue.IsValid());
		});

		It(TEXT("should return no value given an invalid string"), [this]()
		{
			TSharedPtr<FJsonValue> JsonValue =
				JsonStringToJsonValue(TEXT(R"json(a_value)json"));
			TestFalse(TEXT("IsValid"), JsonValue.IsValid());
		});

		It(TEXT("should return a value from a valid value string"), [this]()
		{
			TSharedPtr<FJsonValue> JsonValue =
				JsonStringToJsonValue(TEXT(R"json("hello")json"));
			if (!TestTrue(TEXT("IsValid"), JsonValue.IsValid())) return;
			TestEqual(TEXT("Value"), JsonValue->AsString(), TEXT("hello"));
		});
	});

	Describe(TEXT("JsonToString"), [this]()
	{
		It(TEXT("should return an empty object string from an empty object"), [this]()
		{
			TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			TestEqual(
				TEXT("No JSON object"), JsonToString(JsonObject), TEXT("{}"));
		});

		It(TEXT("should return a JSON object string from a JSON object"), [this]()
		{
			TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetStringField(TEXT("hello"), TEXT("world"));
			TestEqual(
				TEXT("JSON object"),
				JsonToString(JsonObject),
				TEXT(R"json({"hello":"world"})json"));
		});

		It(TEXT("should return a string value from a JSON value string"), [this]()
		{
			TSharedRef<FJsonValue> JsonValue = MakeShared<FJsonValueString>(TEXT("yo"));
			TestEqual(
				TEXT("JSON object"),
				JsonToString(JsonValue),
				FString::Printf(TEXT(R"json("%s")json"), *JsonValue->AsString()));
		});

		It(TEXT("should return an null string from a null JSON value"), [this]()
		{
			TSharedRef<FJsonValue> JsonValue = MakeShared<FJsonValueNull>();
			TestEqual(
				TEXT("JSON object"),
				JsonToString(JsonValue),
				TEXT("null"));
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS