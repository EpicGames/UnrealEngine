// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"

#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/JsonSchema.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(
	FToolsetRegistryJsonSchemaSpec,
	"AI.ToolsetRegistry.JsonSchema",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolsetRegistryJsonSchemaSpec)

namespace
{
	const FString ExpectedDescription = TEXT("a description");
}  // namespace

void FToolsetRegistryJsonSchemaSpec::Define()
{
	using namespace UE::ToolsetRegistry::Internal;

	Describe(TEXT("CreateJsonSchema"), [this]()
	{
		It("can create a schema for an array with no constraints", [this]()
		{
			TestEqual(
				TEXT("JSON schema"),
				JsonToString(CreateJsonSchema(ExpectedDescription, EJson::Array)),
				FString::Printf(
					TEXT(R"json({"description":"%s","type":"array"})json"),
					*ExpectedDescription));
		});

		It("can create a schema for a boolean", [this]()
		{
			TestEqual(
				TEXT("JSON schema"),
				JsonToString(CreateJsonSchema(ExpectedDescription, EJson::Boolean)),
				FString::Printf(
					TEXT(R"json({"description":"%s","type":"boolean"})json"),
					*ExpectedDescription));
		});

		It("can create a schema for null", [this]()
		{
			TestEqual(
				TEXT("JSON schema"),
				JsonToString(CreateJsonSchema(ExpectedDescription, EJson::Null)),
				FString::Printf(
					TEXT(R"json({"description":"%s","type":"null"})json"),
					*ExpectedDescription));
		});

		It("can create a schema for a number", [this]()
		{
			TestEqual(
				TEXT("JSON schema"),
				JsonToString(CreateJsonSchema(ExpectedDescription, EJson::Number)),
				FString::Printf(
					TEXT(R"json({"description":"%s","type":"number"})json"),
					*ExpectedDescription));
		});

		It("can create a schema for a string", [this]()
		{
			TestEqual(
				TEXT("JSON schema"),
				JsonToString(CreateJsonSchema(ExpectedDescription, EJson::String)),
				FString::Printf(
					TEXT(R"json({"description":"%s","type":"string"})json"),
					*ExpectedDescription));
		});

		It("can create a schema for an empty object", [this]()
		{
			TestEqual(
				TEXT("JSON schema"),
				JsonToString(CreateJsonSchema(ExpectedDescription, EJson::Object)),
				FString::Printf(
					TEXT(R"json({"description":"%s","type":"object"})json"),
					*ExpectedDescription));
		});

		It("can create a schema for an object", [this]()
		{
			const FString SpellDescription = TEXT("a spell to cast");
			TSharedRef<FJsonObject> SpellSchema =
				CreateJsonSchema(SpellDescription, EJson::String);
			TSharedRef<FJsonObject> Properties = MakeShared<FJsonObject>();
			Properties->SetObjectField(TEXT("spell"), SpellSchema);
			TestEqual(
				TEXT("JSON schema"),
				JsonToString(CreateJsonSchema(ExpectedDescription, EJson::Object, Properties)),
				FString::Printf(
					TEXT(R"json({"description":"%s","type":"object","properties":%s})json"),
					*ExpectedDescription, *JsonToString(Properties)));
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
