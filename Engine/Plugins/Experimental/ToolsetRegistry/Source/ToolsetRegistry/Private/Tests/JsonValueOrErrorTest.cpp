// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"

#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/JsonSchema.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(
	FToolsetRegistryJsonValueOrErrorSpec,
	"AI.ToolsetRegistry.JsonValueOrError",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolsetRegistryJsonValueOrErrorSpec)

void FToolsetRegistryJsonValueOrErrorSpec::Define()
{
	using namespace UE::ToolsetRegistry;
	using namespace UE::ToolsetRegistry::Internal;

	Describe(TEXT("ToJsonObject"), [this]()
	{
		It(TEXT("should return a value if set"), [this]()
		{
			TSharedRef<FJsonValue> ExpectedValue = MakeShared<FJsonValueString>(TEXT("yo"));
			FJsonValueOrError ValueOrError = MakeValue(ExpectedValue);
			TSharedRef<FJsonObject> JsonObject = ValueOrError.ToJsonObject();
			TestEqual(
				TEXT("JSON object"),
				JsonToString(JsonObject),
				FString::Printf(TEXT(R"json({"%s":%s})json"),
					*FJsonValueOrError::DefaultDescriptor.ValueFieldName,
					*JsonToString(ExpectedValue)));
		});

		It(TEXT("should return null if a null value is set"), [this]()
		{
			FJsonValueOrError ValueOrError = MakeValue(TSharedPtr<FJsonValue>());
			TSharedRef<FJsonObject> JsonObject = ValueOrError.ToJsonObject();
			TestEqual(
				TEXT("JSON object"),
				JsonToString(JsonObject),
				FString::Printf(TEXT(R"json({"%s":null})json"),
					*FJsonValueOrError::DefaultDescriptor.ValueFieldName));
		});

		It(TEXT("should return an error if set"), [this]()
		{
			FString ExpectedError = TEXT("Failed");
			FJsonValueOrError ValueOrError = MakeError(ExpectedError);
			TSharedRef<FJsonObject> JsonObject = ValueOrError.ToJsonObject();
			TestEqual(
				TEXT("JSON object"),
				JsonToString(JsonObject),
				FString::Printf(TEXT(R"json({"%s":"%s"})json"),
					*FJsonValueOrError::DefaultDescriptor.ErrorFieldName,
					*ExpectedError));
		});
	});

	Describe(TEXT("ToJsonString"), [this]()
	{
		It(TEXT("should return a value if set"), [this]()
		{
			FJsonValueOrError ValueOrError = MakeValue(MakeShared<FJsonValueString>(TEXT("yo")));
			TestEqual(
				TEXT("JSON object"),
				JsonToString(ValueOrError.ToJsonObject()),
				ValueOrError.ToJsonString());
		});

		It(TEXT("should return an error if set"), [this]()
		{
			FJsonValueOrError ValueOrError = MakeError(TEXT("Failed"));
			TestEqual(
				TEXT("JSON object"),
				JsonToString(ValueOrError.ToJsonObject()),
				ValueOrError.ToJsonString());
		});
	});

	Describe(TEXT("FromJsonStringOrError"), [this]()
	{
		It(TEXT("should return a value if set"), [this]()
		{
			FString ExpectedValueString = TEXT(R"json("yo")json");
			FJsonValueOrError ValueOrError =
				FJsonValueOrError::FromJsonStringOrError(MakeValue(ExpectedValueString));
			TestEqual(
				TEXT("JSON object"),
				JsonToString(ValueOrError.ToJsonObject()),
				FString::Printf(
					TEXT(R"json({"%s":%s})json"),
					*FJsonValueOrError::DefaultDescriptor.ValueFieldName,
					*ExpectedValueString));
		});


		It(TEXT("should return an error if set"), [this]()
		{
			FString ExpectedError = TEXT("Failed");
			FJsonValueOrError ValueOrError =
				FJsonValueOrError::FromJsonStringOrError(MakeError(ExpectedError));
			TestEqual(
				TEXT("JSON object"),
				JsonToString(ValueOrError.ToJsonObject()),
				FString::Printf(
					TEXT(R"json({"%s":"%s"})json"),
					*FJsonValueOrError::DefaultDescriptor.ErrorFieldName,
					*ExpectedError));
		});
		
		It(TEXT("should fail if provided an invalid JSON string"), [this]()
		{
			FString NotJson = TEXT("not_json");
			FJsonValueOrError ValueOrError =
				FJsonValueOrError::FromJsonStringOrError(MakeValue(NotJson));
			TestEqual(
				TEXT("JSON object"),
				JsonToString(ValueOrError.ToJsonObject()),
				FString::Printf(
					TEXT(R"json({"%s":"Failed to parse JSON result '%s'"})json"),
					*FJsonValueOrError::DefaultDescriptor.ErrorFieldName,
					*NotJson));
		});
	});

	Describe(TEXT("GetJsonSchema"), [this]()
	{
		It(TEXT("should return a value / error schema from an object schema"), [this]()
		{
			TSharedRef<FJsonObject> NestedSchema = CreateJsonSchema(TEXT("magic"), EJson::String);
			TSharedRef<FJsonObject> Schema = FJsonValueOrError::GetJsonSchema(NestedSchema);
			TestEqual(
				TEXT("JSON schema"),
				JsonToString(Schema),
				FString::Printf(
					TEXT(
						R"json({"description":"%s","type":"object","properties":{)json"
						R"json("%s":%s,"%s":{"description":"%s","type":"string"}}})json"),
					*FJsonValueOrError::DefaultDescriptor.ObjectDescription,
					*FJsonValueOrError::DefaultDescriptor.ValueFieldName,
					*JsonToString(NestedSchema),
					*FJsonValueOrError::DefaultDescriptor.ErrorFieldName,
					*FJsonValueOrError::DefaultDescriptor.ErrorFieldDescription));
		});

		It(TEXT("should return a value / error schema with a null value from no schema"), [this]()
		{
			TSharedRef<FJsonObject> Schema =
				FJsonValueOrError::GetJsonSchema(TSharedPtr<FJsonObject>());
			TestEqual(
				TEXT("JSON schema"),
				JsonToString(Schema),
				FString::Printf(
					TEXT(
						R"json({"description":"%s","type":"object","properties":{)json"
						R"json("%s":{"description":"%s","type":"null"},)json"
						R"json("%s":{"description":"%s","type":"string"}}})json"),
					*FJsonValueOrError::DefaultDescriptor.ObjectDescription,
					*FJsonValueOrError::DefaultDescriptor.ValueFieldName,
					*FJsonValueOrError::DefaultDescriptor.NullValueFieldDescription,
					*FJsonValueOrError::DefaultDescriptor.ErrorFieldName,
					*FJsonValueOrError::DefaultDescriptor.ErrorFieldDescription));
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS