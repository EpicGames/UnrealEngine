// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectFunctionToolCallTest.h"

#include "Misc/AutomationTest.h"
#include "Misc/Optional.h"
#include "UObject/Package.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#include "Tests/ToolsetRegistryTestFlags.h"
#include "Tests/ToolCallAsyncResultTest.h"
#include "Tests/UntilFutureCommand.h"
#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/ObjectFunctionToolCall.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

#define DEBUG_PRINT_JSONS_TO_LOG 0

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::ToolsetRegistry
{
	namespace
	{
		void PrintJsonToLog(
			const TCHAR* Name, const TSharedPtr<FJsonObject>& JsonObject)
		{
			if (JsonObject)
			{
				FString JsonString =
					UE::ToolsetRegistry::Internal::JsonToString(JsonObject.ToSharedRef());
				UE_LOGF(LogToolsetRegistry, Display,
					"Json \"%ls\" is valid -\n%ls", Name, *JsonString);
			}
			else
			{
				UE_LOGF(LogToolsetRegistry, Error, "Json \"%ls\" is invalid.", Name);
			}
		}

		// Holds a strong reference to an object instance and an object function tool call.
		template<typename ObjectT>
		struct FStrongObjectFunctionToolCall
		{
			TStrongObjectPtr<ObjectT> InstanceObject;
			TSharedPtr<FObjectFunctionToolCall> ToolCall;
		};

		// Create an object function tool call that wraps a function.
		TSharedPtr<FObjectFunctionToolCall> TestCreateToolCall(
			FAutomationTestBase& Test, TObjectPtr<UObject> InstanceObject,
			const FString& FunctionName)
		{
			auto ToolCall =
				FObjectFunctionToolCall::Create(InstanceObject.Get(), *FunctionName);
			Test.TestTrue(
				FString::Printf(
					TEXT("Construct object function tool call for function %s of %s"),
					*FunctionName, *InstanceObject->GetName()),
				ToolCall.IsValid());
			return ToolCall;
		}

		// Create an object function tool call that wraps a function on an object instance of
		// type ObjectT.
		template<typename ObjectT>
		TOptional<FStrongObjectFunctionToolCall<ObjectT>> TestCreateToolCall(
			FAutomationTestBase& Test, const FString& FunctionName)
		{
			TOptional<FStrongObjectFunctionToolCall<ObjectT>> ToolCallReference;
			auto InstanceObject = NewObject<ObjectT>();
			if (Test.TestTrue(
				FString::Printf(TEXT("Constructed object %s"), *ObjectT::StaticClass()->GetName()),
				InstanceObject != nullptr))
			{
				auto ToolCall = TestCreateToolCall(Test, InstanceObject, FunctionName);
				if (ToolCall)
				{
					ToolCallReference.Emplace();
					ToolCallReference->ToolCall = ToolCall;
					ToolCallReference->InstanceObject = TStrongObjectPtr(InstanceObject);
				}
			}
			return ToolCallReference;
		}
		
		// Ensure a tool call result has no error.
		bool TestHasNoError(FAutomationTestBase& Test, FJsonValueOrError& Result)
		{
			const FString* Error = Result.TryGetError();
			return !Error ||
				Test.TestEqual(TEXT("Tool call result has no error."), *Error, TEXT(""));
		}

		// Ensure a tool call result has a valid value.
		TSharedPtr<FJsonObject> TestHasValue(
			FAutomationTestBase& Test, FJsonValueOrError& Result)
		{
			TSharedPtr<FJsonObject> JsonObject;
			if (TestHasNoError(Test, Result) &&
				Test.TestTrue(TEXT("Tool call result has value."), Result.HasValue()))
			{
				auto JsonValue = Result.GetValue();
				if (Test.TestTrue(TEXT("Tool call result has valid value."), JsonValue.IsValid()))
				{
					JsonObject = JsonValue->AsObject();
#if DEBUG_PRINT_JSONS_TO_LOG
					PrintJsonToLog(TEXT("Tool call result value"), JsonValue.ToSharedRef());
#endif  // DEBUG_PRINT_JSONS_TO_LOG
				}
			}
			return JsonObject;
		}

		// Ensure a tool call result JSON object has a returnValue field.
		const TSharedPtr<FJsonObject> TestHasReturnValueField(
			FAutomationTestBase& Test, FJsonValueOrError& Result, EJson JsonType)	
		{
			const TSharedPtr<FJsonObject> JsonObject = TestHasValue(Test, Result);
			if (JsonObject &&
				!Test.TestTrue(
					TEXT("Tool call result has returnValue."),
					JsonObject->HasTypedField(TEXT("returnValue"), JsonType)))
			{
				return TSharedPtr<FJsonObject>();
			}
			return JsonObject;
		}
	}
}

BEGIN_DEFINE_SPEC(FObjectFunctionToolCallTest, "AI.ToolsetRegistry.ObjectFunctionToolCall",
	ToolsetRegistryTest::Flags)
END_DEFINE_SPEC(FObjectFunctionToolCallTest)

void FObjectFunctionToolCallTest::Define()
{
	using namespace UE::ToolsetRegistry;
	using namespace UE::ToolsetRegistry::Internal;

	Describe(TEXT("ExecuteToolCall"), [this]()
	{
		It(TEXT("should execute a function with a required parameter"), [this]()
		{
			static constexpr float ClassFloatValue = 20.0f;
			static constexpr float FunctionInputFloatValue = 3.0f;

			auto ToolCallRef =
				TestCreateToolCall<UToolCallFakeClass>(*this, TEXT("TestFuncWithRequiredParam"));
			if (!ToolCallRef.IsSet()) return;
			ToolCallRef->InstanceObject->Float = ClassFloatValue;

			// Execute.
			AddCommand(
				FUntilFutureCommand::Create<FJsonValueOrError>(
					ToolCallRef->ToolCall->Execute(
						FObjectFunctionToolCall::FFunctionInputParamsJson(
							TInPlaceType<FString>(), 
							FString::Printf(
								TEXT(R"json({"InFloat": %f})json"), FunctionInputFloatValue))),
					[this, InstanceObject = ToolCallRef->InstanceObject](
						FJsonValueOrError&& ToolCallResult) -> void
					{
						TSharedPtr<FJsonObject> JsonObject =
							TestHasReturnValueField(*this, ToolCallResult, EJson::Number);
						if (!JsonObject) return;
						TestNearlyEqual(
							TEXT("Tool call result value's returnValue"),
							static_cast<float>(JsonObject->GetNumberField(TEXT("returnValue"))),
							InstanceObject->TestFuncWithRequiredParam(FunctionInputFloatValue),
							0.0001f);
					}));
		});

		It(TEXT("should execute a function with an optional parameter"), [this]()
		{
			static constexpr float ClassFloatValue = 20.0f;

			auto ToolCallRef =
				TestCreateToolCall<UToolCallFakeClass>(*this, TEXT("TestFuncWithOptionalParam"));
			if (!ToolCallRef.IsSet()) return;
			ToolCallRef->InstanceObject->Float = ClassFloatValue;

			// Execute.
			AddCommand(
				FUntilFutureCommand::Create<FJsonValueOrError>(
					ToolCallRef->ToolCall->Execute(),
					[this, InstanceObject = ToolCallRef->InstanceObject](
						FJsonValueOrError&& ToolCallResult) -> void
					{
						TSharedPtr<FJsonObject> JsonObject =
							TestHasReturnValueField(*this, ToolCallResult, EJson::Number);
						if (!JsonObject) return;

						TestNearlyEqual(
							TEXT("Tool call result value's returnValue"),
							static_cast<float>(JsonObject->GetNumberField(TEXT("returnValue"))),
							InstanceObject->TestFuncWithOptionalParam(),
							0.0001f);
					}));
		});

		It(TEXT("should execute an asynchronous function"), [this]()
		{
			auto ToolCallRef =
				TestCreateToolCall<UToolCallAsyncResultReturnPropertyTest>(
					*this, TEXT("SlowGreeting"));
			if (!ToolCallRef.IsSet()) return;

			// Execute.
			AddCommand(
				FUntilFutureCommand::Create<FJsonValueOrError>(
					ToolCallRef->ToolCall->Execute(
						FObjectFunctionToolCall::FFunctionInputParamsJson(
							TInPlaceType<FString>(),
							TEXT(R"json({"Subject": "world", "DelayInSeconds": 0.1})json"))),
					[this, InstanceObject = ToolCallRef->InstanceObject](
						FJsonValueOrError&& ToolCallResult) -> void
					{
						TSharedPtr<FJsonObject> JsonObject =
							TestHasReturnValueField(*this, ToolCallResult, EJson::String);
						if (!JsonObject) return;

						TestEqual(
							TEXT("Return value"),
							JsonObject->GetStringField(TEXT("returnValue")),
							TEXT("Hello world"));
					}));
		});

		It(TEXT("should execute an asynchronous function that returns an error"), [this]()
		{
			auto ToolCallRef =
				TestCreateToolCall<UToolCallAsyncResultReturnPropertyTest>(
					*this, TEXT("SlowError"));
			if (!ToolCallRef.IsSet()) return;

			static FString ErrorMessage = TEXT("Failed");
			// Execute.
			AddCommand(
				FUntilFutureCommand::Create<FJsonValueOrError>(
					ToolCallRef->ToolCall->Execute(
						FObjectFunctionToolCall::FFunctionInputParamsJson(
							TInPlaceType<FString>(),
							FString::Printf(
								TEXT(R"json({"Error": "%s", "DelayInSeconds": 0.1})json"),
								*ErrorMessage))),
					[this, InstanceObject = ToolCallRef->InstanceObject](
						FJsonValueOrError&& ToolCallResult) -> void
					{
						if (!TestTrue(TEXT("Has Error"), ToolCallResult.HasError())) return;
						TestEqual(TEXT("Error Matches"), ToolCallResult.GetError(), ErrorMessage);
					}));
		});

		It(TEXT("should fail to execute an asynchronous function that returns null"), [this]()
		{
			auto ToolCallRef =
				TestCreateToolCall<UToolCallAsyncResultReturnPropertyTest>(
					*this, TEXT("Broken"));
			if (!ToolCallRef.IsSet()) return;

			// Execute.
			AddCommand(
				FUntilFutureCommand::Create<FJsonValueOrError>(
					ToolCallRef->ToolCall->Execute(),
					[this, InstanceObject = ToolCallRef->InstanceObject](
						FJsonValueOrError&& ToolCallResult) -> void
					{
						if (!TestTrue(TEXT("Has Error"), ToolCallResult.HasError())) return;
						TestEqual(
							TEXT("Error Matches"),
							ToolCallResult.GetError(),
							TEXT("Broken failed to return a UToolCallAsyncResult instance."));
					}));
		});
	});
}

#endif

PRAGMA_ENABLE_DEPRECATION_WARNINGS


