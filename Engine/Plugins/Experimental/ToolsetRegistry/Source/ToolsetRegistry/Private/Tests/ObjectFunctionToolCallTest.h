// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ObjectFunctionToolCallTest.generated.h"

UCLASS(BlueprintType)
class UToolCallFakeClass : public UObject
{
	GENERATED_BODY()
	
public:
	
	UPROPERTY(EditAnywhere, Category="Main")
	float Float = 0.0f;
	
	UFUNCTION(BlueprintCallable, Category="Main") 
	float TestFuncWithRequiredParam(const float InFloat) const 
	{
		return 100.0f + Float + InFloat; 
	}
	
	UFUNCTION(BlueprintCallable, Category="Main") 
	float TestFuncWithOptionalParam(const float InFloat = 3.0f) const 
	{
		return 100.0f + Float + InFloat; 
	}
};
