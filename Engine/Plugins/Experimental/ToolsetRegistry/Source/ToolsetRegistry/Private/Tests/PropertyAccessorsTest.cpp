// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PropertyAccessorsTest.h"

#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "UObject/Class.h"

#include "ToolsetRegistry/PropertyAccessors.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FToolsetRegistryPropertyAccessorsTest,
	"AI.ToolsetRegistry.PropertyAccessors",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolsetRegistryPropertyAccessorsTest)

void FToolsetRegistryPropertyAccessorsTest::Define()
{
	using namespace UE::ToolsetRegistry::Internal;

	Describe(TEXT("PropertyValueAsObject"), [this]()
	{
		It(TEXT("Should return a matching object property for null value"), [this]()
		{
			FProperty* Property =
				FToolsetRegistryPropertyValueAsObjectTestStruct::FindTestObjectProperty(*this);
			if (!Property) return;

			FToolsetRegistryPropertyValueAsObjectTestStruct TestStruct;
			auto MaybeObject =
				PropertyValueAsObject<UToolsetRegistryPropertyAccessorsTestObject>(
					Property, &TestStruct.TestObject);
			if (!TestTrue(TEXT("Matches object"), MaybeObject.IsSet())) return;
			TestTrue(TEXT("Matches value"), (*MaybeObject).Get() == TestStruct.TestObject.Get());
		});

		It(TEXT("Should return a matching object property for non-null value"), [this]()
		{
			FProperty* Property =
				FToolsetRegistryPropertyValueAsObjectTestStruct::FindTestObjectProperty(*this);
			if (!Property) return;

			FToolsetRegistryPropertyValueAsObjectTestStruct TestStruct;
			TestStruct.TestObject = NewObject<UToolsetRegistryPropertyAccessorsTestObject>();
			auto MaybeObject =
				PropertyValueAsObject<UToolsetRegistryPropertyAccessorsTestObject>(
					Property, &TestStruct.TestObject);
			if (!TestTrue(TEXT("Matches object"), MaybeObject.IsSet())) return;
			TestTrue(TEXT("Matches value"), (*MaybeObject).Get() == TestStruct.TestObject.Get());
		});

		It(TEXT("Should not return a non-matching object property value"), [this]()
		{
			FProperty* Property =
				FToolsetRegistryPropertyValueAsObjectTestStruct::FindTestAssetManager(*this);
			if (!Property) return;

			FToolsetRegistryPropertyValueAsObjectTestStruct TestStruct;
			auto MaybeObject =
				PropertyValueAsObject<UToolsetRegistryPropertyAccessorsTestObject>(
					Property, &TestStruct.TestAssetManager);
			TestFalse(TEXT("Does not match object"), MaybeObject.IsSet());
		});

		It(TEXT("Should not return a non-matching property value"), [this]()
		{
			FProperty* Property =
				FToolsetRegistryPropertyValueAsObjectTestStruct::FindIntProperty(*this);
			if (!Property) return;

			FToolsetRegistryPropertyValueAsObjectTestStruct TestStruct;
			auto MaybeObject =
				PropertyValueAsObject<UToolsetRegistryPropertyAccessorsTestObject>(
					Property, &TestStruct.TestObject);
			TestFalse(TEXT("Does not match object"), MaybeObject.IsSet());
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS