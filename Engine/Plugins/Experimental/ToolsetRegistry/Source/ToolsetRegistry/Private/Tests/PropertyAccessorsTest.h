// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Engine/AssetManager.h"
#include "Misc/AutomationTest.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "PropertyAccessorsTest.generated.h"

UCLASS(BlueprintType, MinimalAPI, Hidden)
class UToolsetRegistryPropertyAccessorsTestObject : public UObject
{
	GENERATED_BODY()
};

USTRUCT(BlueprintType)
struct FToolsetRegistryPropertyValueAsObjectTestStruct
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UToolsetRegistryPropertyAccessorsTestObject> TestObject;

	UPROPERTY()
	TObjectPtr<UAssetManager> TestAssetManager;

	UPROPERTY()
	int TestInt = 0;

public:
	static FProperty* FindPropertyByName(
		FAutomationTestBase& Test, const FString& PropertyName)
	{
		FProperty* Property = StaticStruct()->FindPropertyByName(*PropertyName);
		if (!Test.TestTrue(
				FString::Printf(TEXT("Has %s"), *PropertyName),
				Property != nullptr))
		{
			return nullptr;
		}
		return Property;
	}

	static FProperty* FindTestObjectProperty(FAutomationTestBase& Test)
	{
		return FindPropertyByName(Test, TEXT("TestObject"));
	}

	static FProperty* FindTestAssetManager(FAutomationTestBase& Test)
	{
		return FindPropertyByName(Test, TEXT("TestAssetManager"));
	}

	static FProperty* FindIntProperty(FAutomationTestBase& Test)
	{
		return FindPropertyByName(Test, TEXT("TestInt"));
	}
};