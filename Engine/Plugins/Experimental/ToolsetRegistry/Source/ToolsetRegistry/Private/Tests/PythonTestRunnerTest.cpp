// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "ToolsetRegistry/PythonTestRunner.h"

#if WITH_AUTOMATION_TESTS

class FPythonAutomationTestAccessor
{
public:
	static TValueOrError<TArray<FString>, FString> ParseJsonStringArray(
		const FString& StringArrayJson)
	{
		return UE::ToolsetRegistry::FPythonAutomationTest::ParseJsonStringArray(StringArrayJson);
	}

	static TValueOrError<FString, FString> RunTestRunnerPythonCode(
		const FString& CodeToRun, const FString& ErrorContext)
	{
		return UE::ToolsetRegistry::FPythonAutomationTest::RunTestRunnerPythonCode(
			CodeToRun, ErrorContext);
	}

	static TValueOrError<FString, FString> StartTestExecution(
		UE::ToolsetRegistry::FPythonAutomationTest& Test, const FString& TestId)
	{
		return Test.StartTestExecution(TestId);
	}

	static bool PollForTestCompletion(
		UE::ToolsetRegistry::FPythonAutomationTest& Test, const FString& TestId)
	{
		return Test.PollForTestCompletion(TestId);
	}
};

class FPythonTestRunnerAccessor
{
public:
	static void SetRunTestDelegate(
		TStrongObjectPtr<UPythonTestRunner> Runner,
		TFunction<void(const FString&)>&& RunTestDelegate)
	{
		Runner->RunTestDelegate = MoveTemp(RunTestDelegate);
	}
};

BEGIN_DEFINE_SPEC(
	FPythonTestRunnerTest,
	"AI.ToolsetRegistry.PythonTestRunnerTest",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
private:
	// Get search options for test cases used by unittest_runner.
	static FPythonTestRunnerSearchOptions GetSearchOptions(const FString& TestNameGlob)
	{
		FPythonTestRunnerSearchOptions Options;
		Options.RootModule = PythonTestsRootModule;
		Options.TestModuleGlob = PythonTestModule;
		Options.TestNameGlobs.Add(TestNameGlob);
		return Options;
	}

	// Create a Python automation test that discovers unittest_runner test cases.
	static TSharedPtr<UE::ToolsetRegistry::FPythonAutomationTest> CreatePythonAutomationTest(
		const FString& TestNameGlob)
	{
		return MakeShared<UE::ToolsetRegistry::FPythonAutomationTest>(
			TestBaseName, GetSearchOptions(TestNameGlob));
	}

	// Create a Python test runner.
	TStrongObjectPtr<UPythonTestRunner> CreatePythonTestRunner(
		const FString& TestNameGlob)
	{
		return TStrongObjectPtr<UPythonTestRunner>(
			UPythonTestRunner::Create(TestBaseName, GetSearchOptions(TestNameGlob)));
	}
	
private:
	// Ensure that a result has an error and the error string is not empty.
	template<typename ValueT>
	bool HasError(const TValueOrError<ValueT, FString>& MaybeError)
	{
		if (!TestTrue(TEXT("HasError"), MaybeError.HasError())) return false;
		return TestNotEqual(TEXT("ErrorString"), MaybeError.GetError(), TEXT(""));
	}

	// Clear last test results to reset any global state.
	void ClearLastTestResults()
	{
		TestTrue(
			TEXT("Clear results"),
			FPythonAutomationTestAccessor::RunTestRunnerPythonCode(
				TEXT("unittest_runner.clear_last_test_results()"),
				TEXT("Setup test case")).HasValue());
	}

	// Expect a UPythonTestRunner not created error.
	void ExpectPythonTestRunnerNotCreatedError()
	{
		AddExpectedError(
			TEXT("must be instanced using create"),
			EAutomationExpectedMessageFlags::MatchType::Contains, 1, true);
	}

	// Expect Python error logs.
	void ExpectPythonErrors()
	{
		// Python logs multiple error lines so ignore them all.
		AddExpectedError(
			TEXT(".*"),
			EAutomationExpectedMessageFlags::MatchType::Contains, 0, true);
	}

private:
	static const TCHAR TestBaseName[];
	static const TCHAR PythonTestsRootModule[];
	static const TCHAR PythonTestModule[];
	static const TCHAR PythonTestSuccessId[];
	static const TCHAR PythonTestFailId[];
	END_DEFINE_SPEC(FPythonTestRunnerTest)

const TCHAR FPythonTestRunnerTest::TestBaseName[] = TEXT("PythonAutomationTest");
const TCHAR FPythonTestRunnerTest::PythonTestsRootModule[] =
	TEXT("toolset_registry.tests");
const TCHAR FPythonTestRunnerTest::PythonTestModule[] =
	TEXT("_unittest_runner_test_cases");
const TCHAR FPythonTestRunnerTest::PythonTestSuccessId[] =
	TEXT("toolset_registry.tests._unittest_runner_test_cases.TestSuccess.test_succeed");
const TCHAR FPythonTestRunnerTest::PythonTestFailId[] =
	TEXT("toolset_registry.tests._unittest_runner_test_cases.TestFailure.test_fail");

void FPythonTestRunnerTest::Define()
{
	using namespace UE::ToolsetRegistry;

	Describe(TEXT("FPythonTestRunnerSearchOptions"), [this]()
	{
		Describe(TEXT("BuildPythonCode"), [this]()
		{
			It(TEXT("Returns code for empty object"), [this]()
			{
				FPythonTestRunnerSearchOptions Options;
				TValueOrError<FString, FString> MaybePythonCode = Options.BuildPythonCode();
				if (!TestFalse(TEXT("HasError"), MaybePythonCode.HasError())) return;
				TestEqual(
					TEXT("Code"),
					*MaybePythonCode.GetValue(),
					TEXT("unittest_runner.TestSearchOptions()"));
			});

			It(TEXT("Returns code with a root module"), [this]()
			{
				FPythonTestRunnerSearchOptions Options;
				Options.RootModule = "foo.bar";
				TValueOrError<FString, FString> MaybePythonCode = Options.BuildPythonCode();
				if (!TestFalse(TEXT("HasError"), MaybePythonCode.HasError())) return;
				TestEqual(
					TEXT("Code"),
					*MaybePythonCode.GetValue(),
					TEXT("unittest_runner.TestSearchOptions(root_module='foo.bar')"));
			});

			It(TEXT("Fails with invalid root module"), [this]()
			{
				FPythonTestRunnerSearchOptions Options;
				Options.RootModule = "foo'bar";
				HasError(Options.BuildPythonCode());
			});

			It(TEXT("Returns code with module glob"), [this]()
			{
				FPythonTestRunnerSearchOptions Options;
				Options.TestModuleGlob = "test_things*";
				TValueOrError<FString, FString> MaybePythonCode = Options.BuildPythonCode();
				if (!TestFalse(TEXT("HasError"), MaybePythonCode.HasError())) return;
				TestEqual(
					TEXT("Code"),
					*MaybePythonCode.GetValue(),
					TEXT("unittest_runner.TestSearchOptions(test_module_glob='test_things*')"));
			});

			It(TEXT("Fails with invalid module glob"), [this]()
			{
				FPythonTestRunnerSearchOptions Options;
				Options.TestModuleGlob = "'test_things'";
				HasError(Options.BuildPythonCode());
			});

			It(TEXT("Returns code with test glob"), [this]()
			{
				FPythonTestRunnerSearchOptions Options;
				Options.TestNameGlobs.Add("test_foo");
				Options.TestNameGlobs.Add("test_bar");
				TValueOrError<FString, FString> MaybePythonCode = Options.BuildPythonCode();
				if (!TestFalse(TEXT("HasError"), MaybePythonCode.HasError())) return;
				TestEqual(
					TEXT("Code"),
					*MaybePythonCode.GetValue(),
					TEXT(
						"unittest_runner.TestSearchOptions("
						"test_name_globs=set(['test_foo', 'test_bar']))"));
			});

			It(TEXT("Fails with invalid test glob"), [this]()
			{
				FPythonTestRunnerSearchOptions Options;
				Options.TestNameGlobs.Add("test_foo");
				Options.TestNameGlobs.Add("test'bar");
				TValueOrError<FString, FString> MaybePythonCode = Options.BuildPythonCode();
				HasError(Options.BuildPythonCode());
			});

			It(TEXT("Returns code with multiple args"), [this]()
			{
				FPythonTestRunnerSearchOptions Options;
				Options.RootModule = "foo.bar";
				Options.TestModuleGlob = "test_*";
				TValueOrError<FString, FString> MaybePythonCode = Options.BuildPythonCode();
				if (!TestFalse(TEXT("HasError"), MaybePythonCode.HasError())) return;
				TestEqual(
					TEXT("Code"),
					*MaybePythonCode.GetValue(),
					TEXT(
						"unittest_runner.TestSearchOptions("
						"root_module='foo.bar', test_module_glob='test_*')"));
			});
		});
	});

	Describe(TEXT("FPythonAutomationTest"), [this]()
	{
		Describe(TEXT("ParseJsonStringArray"), [this]()
		{
			It(TEXT("Can parse a list of strings"), [this]()
			{
				TValueOrError<TArray<FString>, FString> MaybeResult =
					FPythonAutomationTestAccessor::ParseJsonStringArray(
						TEXT(R"json(["a", "bc"])json"));
				if (!TestTrue(TEXT("Successful"), MaybeResult .HasValue())) return;

				TArray<FString> Result = MaybeResult.StealValue();
				TestEqual(TEXT("Number of items"), Result.Num(), 2);
				TestEqual(TEXT("Items"), *FString::Join(Result, TEXT(", ")), TEXT("a, bc"));
			});

			It(TEXT("Fails to parse non-list"), [this]()
			{
				HasError(FPythonAutomationTestAccessor::ParseJsonStringArray(TEXT(R"json({})json")));
			});

			It(TEXT("Fails to parse list with non-strings"), [this]()
			{
				HasError(
					FPythonAutomationTestAccessor::ParseJsonStringArray(
						TEXT(R"json(["a", 2)json")));
			});
		});

		Describe(TEXT("RunTestRunnerPythonCode"), [this]()
		{
			static const TCHAR ErrorContext[] = TEXT("Testing");

			It(TEXT("Can run a valid expression"), [this]()
			{
				TValueOrError<FString, FString> MaybeResult =
					FPythonAutomationTestAccessor::RunTestRunnerPythonCode(
						TEXT("json.dumps(1)"), ErrorContext);
				if (!TestTrue(TEXT("Successful"), MaybeResult .HasValue())) return;

				TestEqual(TEXT("Result"), *MaybeResult.GetValue(), TEXT("1"));
			});

			It(TEXT("Fails with a invalid expression"), [this]()
			{
				ExpectPythonErrors();
				TValueOrError<FString, FString> MaybeResult =
					FPythonAutomationTestAccessor::RunTestRunnerPythonCode(
						TEXT("1 = 2"), ErrorContext);
				if (!TestTrue(TEXT("Fails"), MaybeResult.HasError())) return;
				TestTrue(
					TEXT("Error contains context"),
					MaybeResult.GetError().Contains(ErrorContext));
			});
		});

		Describe(TEXT("GetTests"), [this]
		{
			It(TEXT("Returns discovered tests"), [this]()
			{
				TSharedPtr<FPythonAutomationTest> AutomationTest =
					CreatePythonAutomationTest(PythonTestSuccessId);
				TArray<FString> TestIds;
				TArray<FString> BeautifiedNames;
				AutomationTest->GetTests(BeautifiedNames, TestIds);
				if (!TestEqual(
						TEXT("TestIds count matches name count"),
						TestIds.Num(), BeautifiedNames.Num()))
				{
					return;
				}

				FString CommaSeparatedTestIds = FString::Join(TestIds, TEXT(", "));
				TestEqual(TEXT("TestIds"), CommaSeparatedTestIds, PythonTestSuccessId);
				TestEqual(
					TEXT("TestIds match beautified names"),
					CommaSeparatedTestIds,
					FString::Join(BeautifiedNames, TEXT(", ")));
			});
		});

		Describe(TEXT("TestExecution"), [this]()
		{
			static const TCHAR InvalidTestId[] = TEXT("invalid',test.id");
			constexpr float TimeoutInSeconds = 10.0f;

			BeforeEach([this]()
			{
				ClearLastTestResults();
			});

			Describe(TEXT("GetLastTestResult"), [this]()
			{
				It(TEXT("Returns nothing when a test isn't runnning"), [this]()
				{
					TSharedPtr<FPythonAutomationTest> AutomationTest =
						CreatePythonAutomationTest(PythonTestSuccessId);
					TValueOrError<TOptional<bool>, FString> MaybeResult =
						AutomationTest->GetLastTestResult(PythonTestSuccessId);
					if (!TestTrue(TEXT("Has result"), MaybeResult.HasValue())) return;
					TestFalse(TEXT("Not running"), MaybeResult.GetValue().IsSet());
				});

				It(TEXT("Returns a result when a test is complete"), [this]()
				{
					TSharedPtr<FPythonAutomationTest> AutomationTest =
						CreatePythonAutomationTest(PythonTestSuccessId);
					TValueOrError<FString, FString> MaybeTestId =
						FPythonAutomationTestAccessor::StartTestExecution(
							*AutomationTest, PythonTestSuccessId);
					if (!TestTrue(TEXT("Has test ID"), MaybeTestId.HasValue())) return;

					AddCommand(
						new FUntilCommand(
							[this, AutomationTest, TestId = MaybeTestId.StealValue()]()
							{
								return FPythonAutomationTestAccessor::PollForTestCompletion(
									*AutomationTest, TestId);
							},
							TimeoutInSeconds));
				});

				It(TEXT("Fails with an invalid test ID"), [this]()
				{
					TSharedPtr<FPythonAutomationTest> AutomationTest =
						CreatePythonAutomationTest(PythonTestSuccessId);
					HasError(AutomationTest->GetLastTestResult(InvalidTestId));
				});

				It(TEXT("Returns a failure when a test is fails"), [this]()
				{
					TSharedPtr<FPythonAutomationTest> AutomationTest =
						CreatePythonAutomationTest(PythonTestFailId);
					ExpectPythonErrors();
					TValueOrError<FString, FString> MaybeTestId =
						FPythonAutomationTestAccessor::StartTestExecution(
							*AutomationTest, PythonTestFailId);
					if (!TestTrue(TEXT("Has test ID"), MaybeTestId.HasValue())) return;

					AddCommand(
						new FUntilCommand(
							[this, AutomationTest, TestId = MaybeTestId.StealValue()]()
							{
								auto MaybeResult = AutomationTest->GetLastTestResult(TestId);
								if (MaybeResult.HasError()) return false;
								if (!MaybeResult.GetValue().IsSet()) return false;
								TestFalse(TEXT("Failed"), MaybeResult.GetValue().GetValue());
								return true;
							},
							TimeoutInSeconds));
				});
			});

			Describe(TEXT("RunTest"), [this]
			{
				It(TEXT("Can run a test to completion"), [this]()
				{
					TSharedPtr<FPythonAutomationTest> AutomationTest =
						CreatePythonAutomationTest(PythonTestSuccessId);
					AutomationTest->RunTest(PythonTestSuccessId);
				});
			});
		});
	});

	Describe(TEXT("UPythonTestRunner"), [this]()
	{
		BeforeEach([this]()
		{
			ClearLastTestResults();
		});

		Describe(TEXT("GetTests"), [this]()
		{
			It(TEXT("Fails with a non-created instance"), [this]()
			{
				TStrongObjectPtr<UPythonTestRunner> Runner(NewObject<UPythonTestRunner>());
				ExpectPythonTestRunnerNotCreatedError();
				(void)Runner->GetTests();
			});

			It(TEXT("Returns discovered tests"), [this]()
			{
				auto Runner = CreatePythonTestRunner(PythonTestSuccessId);
				TestEqual(
					TEXT("TestIds"),
					FString::Join(Runner->GetTests(), TEXT(", ")),
					PythonTestSuccessId);
			});
		});

		Describe(TEXT("TestExecution"), [this]()
		{

			Describe(TEXT("GetLastTestResult"), [this]()
			{
				It(TEXT("Fails with a non-created instance"), [this]()
				{
					TStrongObjectPtr<UPythonTestRunner> Runner(NewObject<UPythonTestRunner>());
					ExpectPythonTestRunnerNotCreatedError();
					(void)Runner->GetLastTestResult(PythonTestSuccessId);
				});

				It(TEXT("Return an incomplete result for a non-executed test"), [this]()
				{
					TStrongObjectPtr<UPythonTestRunner> Runner =
						CreatePythonTestRunner(PythonTestSuccessId);
					FPythonTestRunnerResult Result =
						Runner->GetLastTestResult(PythonTestSuccessId);
					TestFalse(TEXT("IsComplete"), Result.bIsComplete);
					TestFalse(TEXT("IsSuccessful"), Result.bIsSuccessful);
				});
			});

			Describe(TEXT("RunTest"), [this]()
			{
				It(TEXT("Fails with a non-created instance"), [this]()
				{
					TStrongObjectPtr<UPythonTestRunner> Runner(NewObject<UPythonTestRunner>());
					ExpectPythonTestRunnerNotCreatedError();
					Runner->RunTest(PythonTestSuccessId);
				});

				It(TEXT("Can run a test"), [this]()
				{
					TStrongObjectPtr<UPythonTestRunner> Runner =
						CreatePythonTestRunner(PythonTestSuccessId);
					FString TestNameToRun;
					FPythonTestRunnerAccessor::SetRunTestDelegate(
						Runner, 
						[&TestNameToRun](const FString& ReceivedTestNameToRun)
						{
							TestNameToRun = ReceivedTestNameToRun;
						});
					Runner->RunTest(PythonTestSuccessId);
					TestEqual(
						TEXT("Test Name"),
						TestNameToRun,
						FString::Printf(TEXT("%s.%s"), TestBaseName, PythonTestSuccessId));
				});
			});
		});
	});
}

#endif  // WITH_AUTOMATION_TESTS
