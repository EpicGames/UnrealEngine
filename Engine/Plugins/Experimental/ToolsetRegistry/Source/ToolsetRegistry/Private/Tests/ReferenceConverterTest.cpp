// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceConverterTest.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CoreMinimal.h"
#include "Engine/Blueprint.h"
#include "Misc/AutomationTest.h"

#include "ToolsetRegistry/JsonSchema.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"
#include "ToolsetRegistry/ToolsetJson.h"
#include "Tests/ToolCallTestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FToolsetReferenceConverterSpec, "AI.ToolsetRegistry.ReferenceConverterSpec",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	void TestObjectRefSchema(const TSharedPtr<FJsonObject>& Schema, UClass* Class, bool IsClass);
	void TestObjectRefData(const TSharedPtr<FJsonObject>& Schema, const FString& Expected);
END_DEFINE_SPEC(FToolsetReferenceConverterSpec)

void FToolsetReferenceConverterSpec::TestObjectRefSchema(
	const TSharedPtr<FJsonObject>& Schema, UClass* Class, bool IsClass)
{
	if (!TestTrue("Schema", Schema.IsValid())) return;

	FString Type;
	TestTrue("Type", Schema->TryGetStringField(TEXT("type"), Type));
	TestEqual("Type", Type, FString(TEXT("object")));

	if (Class)
	{
		FString Title;
		TestTrue("Title", Schema->TryGetStringField(TEXT("title"), Title));
		FString ClassPrefix(IsClass ? TEXT("Class@") : TEXT(""));
		TestEqual("Title", Title, ClassPrefix + Class->GetPathName());
	}

	const TSharedPtr<FJsonObject>* Properties = nullptr;
	if (!(TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties)) &&
		Properties))
	{
		return;
	}
	const TSharedPtr<FJsonObject>* RefPath = nullptr;
	if (!(TestTrue("RefPath", Properties->Get()->TryGetObjectField(TEXT("refPath"), RefPath)) &&
		RefPath))
	{
		return;
	}
	TestTrue("RefPath", RefPath->Get()->TryGetStringField(TEXT("type"), Type));
	TestEqual("RefPath", Type, FString(TEXT("string")));
}

void FToolsetReferenceConverterSpec::TestObjectRefData(
	const TSharedPtr<FJsonObject>& Schema, const FString& Expected)
{
	if (!TestTrue("Schema", Schema.IsValid())) return;

	FString RefPath;
	TestTrue("Unpack", Schema->TryGetStringField(TEXT("refPath"), RefPath));
	TestEqual("RefPath", RefPath, Expected);
}

void FToolsetReferenceConverterSpec::Define()
{
	using namespace UE::ToolsetRegistry::Internal;
	using namespace UE::ToolsetRegistry::Internal::ToolsetJson;

	It("Converts UObject* property to reference schema", [this]()
	{
		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestObject");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Prop);
		TestObjectRefSchema(Schema, UReferenceConverterTestObject::StaticClass(), false);
	});

	It("Converts UObject* property to and from JSON", [this]()
	{
		FReferenceConverterTest TestIn;
		TObjectPtr<UReferenceConverterTestObject> TestOther = NewObject<UReferenceConverterTestObject>();
		TestIn.TestObject = TestOther;

		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestObject");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestObject);
		TestObjectRefData(OutJson->AsObject(), TestOther->GetPathName());

		FReferenceConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestObject));
		TestEqual("RefCheck", TestIn.TestObject, TestOut.TestObject);
	});

	It("Ensures refs can be set from all forms of null", [this]()
	{
		FReferenceConverterTest Test;
		TObjectPtr<UReferenceConverterTestObject> TestOther = NewObject<UReferenceConverterTestObject>();
		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestObject");
		TSharedPtr<FJsonObject> JsonRef = MakeShared<FJsonObject>();

		// Null Json type.
		Test.TestObject = TestOther;
		TestEqual("RefCheck", Test.TestObject, TestOther);
		JsonRef->SetField(FString(TEXT("refPath")), MakeShared<FJsonValueNull>());
		TestTrue("Output", JsonDataToProperty(
			MakeShared<FJsonValueObject>(JsonRef), Property, &Test.TestObject));
		TestEqual("RefCheck", Test.TestObject, TObjectPtr<UReferenceConverterTestObject>());

		// None is the UE standard.
		Test.TestObject = TestOther;
		TestEqual("RefCheck", Test.TestObject, TestOther);
		JsonRef->SetStringField(FString(TEXT("refPath")), FString(TEXT("None")));
		TestTrue("Output", JsonDataToProperty(
			MakeShared<FJsonValueObject>(JsonRef), Property, &Test.TestObject));
		TestEqual("RefCheck", Test.TestObject, TObjectPtr<UReferenceConverterTestObject>());

		// Empty string also means null.
		Test.TestObject = TestOther;
		TestEqual("RefCheck", Test.TestObject, TestOther);
		JsonRef->SetStringField(FString(TEXT("refPath")), FString(TEXT("")));
		TestTrue("Output", JsonDataToProperty(
			MakeShared<FJsonValueObject>(JsonRef), Property, &Test.TestObject));
		TestEqual("RefCheck", Test.TestObject, TObjectPtr<UReferenceConverterTestObject>());

		// And we also support null just in case.
		Test.TestObject = TestOther;
		TestEqual("RefCheck", Test.TestObject, TestOther);
		JsonRef->SetStringField(FString(TEXT("refPath")), FString(TEXT("null")));
		TestTrue("Output", JsonDataToProperty(
			MakeShared<FJsonValueObject>(JsonRef), Property, &Test.TestObject));
		TestEqual("RefCheck", Test.TestObject, TObjectPtr<UReferenceConverterTestObject>());
	});

	It("Accepts null for optional parameter", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestOptionalObject"), TEXT(R"({"testObject": {"refPath": ""}})"));
		TestFalse("Should not have error", Result.HasError());
	});

	It("Rejects null for non-optional object properties", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestObjectParam"),
				TEXT(R"({"testObject": {"refPath": ""}})"));

		if (!TestTrue("Should have error", Result.HasError()))
		{
			return;
		}
		TestTrue("Error message", Result.GetError().Contains(TEXT("None is not valid value for property")));
	});

	It("Rejects invalid object path in object properties", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestObjectParam"),
				TEXT(R"({"testObject": {"refPath": "/Invalid/Path/DoesNotExist.DoesNotExist"}})"));
		if (!TestTrue("Should have error", Result.HasError()))
		{
			return;
		}
		TestTrue("Error contains validation message",
				 Result.GetError().Contains(TEXT("is not valid Object for property")));
	});

	It("Rejects wrong object type in object properties", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestSpecObjectParam"),
				FString::Printf(
					TEXT(R"({"testObject": {"refPath": "%s"}})"), *UObject::StaticClass()->GetPathName()));
		if (!TestTrue("Should have error", Result.HasError()))
		{
			return;
		}
		TestTrue("Error contains validation message",
				 Result.GetError().Contains(TEXT("is not valid ReferenceConverterTestObject for property")));
	});

	It("Rejects invalid path for class properties", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestClassParam"),
				TEXT(R"({"testClass": {"refPath": "/Invalid/Class/Path.Path"}})"));
		if (!TestTrue("Should have error", Result.HasError()))
		{
			return;
		}
		TestTrue("Error contains validation message",
				 Result.GetError().Contains(TEXT("is not valid Class for property")));
	});

	It("Rejects non-class object for class properties", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> TestObj = NewObject<UReferenceConverterTestObject>();
		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestClassParam"),
				FString::Printf(TEXT(R"({"testClass": {"refPath": "%s"}})"), *TestObj->GetPathName()));
		if (!TestTrue("Should have error", Result.HasError()))
		{
			return;
		}
		TestTrue("Error contains validation message",
				 Result.GetError().Contains(TEXT("is not valid Class for property")));
	});

	It("Rejects class not matching metaclass constraint in properties", [this]()
	{
		FString WrongClassPath = UObject::StaticClass()->GetPathName();
		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestSubClassParam"),
				FString::Printf(TEXT(R"({"testClass": {"refPath": "%s"}})"), *WrongClassPath));
		if (!TestTrue("Should have error", Result.HasError()))
		{
			return;
		}
		TestTrue("Error contains validation message",
				 Result.GetError().Contains(TEXT("is not valid ReferenceConverterTestObject for property")));
	});

	It("Rejects wrong type for soft object reference", [this]()
	{
		FString InvalidJson = FString::Printf(
			TEXT(R"({"testObject": {"refPath": "%s"}})"),
			*UObject::StaticClass()->GetPathName());

		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestSoftObjectParam"), InvalidJson);

		if (!TestTrue("Should have error", Result.HasError())) return;
		TestTrue("Error message",
			Result.GetError().Contains(TEXT("is not valid ReferenceConverterTestObject")));
	});

	It("Allows valid soft object reference", [this]()
	{
		UReferenceConverterTestObject* TestObj = NewObject<UReferenceConverterTestObject>();
		FString ValidJson = FString::Printf(
			TEXT(R"({"testObject": {"refPath": "%s"}})"),
			*TestObj->GetPathName());

		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestSoftObjectParam"), ValidJson);

		TestFalse("Should not have error", Result.HasError());
	});

	It("Accepts Blueprint asset path for FSoftClassPath parameter", [this]()
	{
		// Regression test: before the fix, FSoftClassPath validation called AssetData.GetClass()
		// which returns UBlueprint for Blueprint assets. Since UBlueprint is not a child of UClass,
		// it incorrectly raised a script error. The fix adds an explicit check for UBlueprint.

		// Create a transient Blueprint and register it with the Asset Registry so that
		// AssetData.IsValid() is true, which is required to exercise the validation branch.
		UBlueprint* Blueprint = NewObject<UBlueprint>(
			GetTransientPackage(), TEXT("TestBlueprintForSoftClassPathValidation"), RF_Transient);
		Blueprint->AddToRoot();

		FAssetRegistryModule& AssetRegistryModule =
			FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		AssetRegistry.AssetCreated(Blueprint);

		ON_SCOPE_EXIT
		{
			AssetRegistry.AssetDeleted(Blueprint);
			Blueprint->RemoveFromRoot();
		};

		FString BlueprintPath = Blueprint->GetPathName();
		FString Json = FString::Printf(
			TEXT(R"({"testClassPath": {"refPath": "%s"}})"), *BlueprintPath);

		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestSoftClassPathParam"), Json);

		TestFalse("Blueprint asset path should be accepted for FSoftClassPath", Result.HasError());
	});

	It("Rejects non-class for soft class reference", [this]()
	{
		UReferenceConverterTestObject* TestObj = NewObject<UReferenceConverterTestObject>();
		FString InvalidJson = FString::Printf(
			TEXT(R"({"testClass": {"refPath": "%s"}})"),
			*TestObj->GetPathName());

		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestSoftClassParam"), InvalidJson);
		if (!TestTrue("Should have error", Result.HasError())) return;
		TestTrue("Error message",
			Result.GetError().Contains(TEXT("is not valid Class for property")));
	});

	It("Rejects wrong class type for soft class reference", [this]()
	{
		FString InvalidJson = FString::Printf(
			TEXT(R"({"testClass": {"refPath": "%s"}})"),
			*UObject::StaticClass()->GetPathName());

		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestSoftClassParam"), InvalidJson);

		if (!TestTrue("Should have error", Result.HasError())) return;
		TestTrue("Error message",
			Result.GetError().Contains(TEXT("is not valid ReferenceConverterTestObject")));
	});

	It("Allows valid soft class reference", [this]()
	{
		FString ValidJson = FString::Printf(
			TEXT(R"({"testClass": {"refPath": "%s"}})"),
			*UReferenceConverterTestObject::StaticClass()->GetPathName());

		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestSoftClassParam"), ValidJson);

		TestFalse("Should not have error", Result.HasError());
	});

	It("Converts SoftPath property to reference schema", [this]()
	{
		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestObjectPath");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Prop);
		TestObjectRefSchema(Schema, UObject::StaticClass(), false);
	});

	It("Converts SoftPath property to and from JSON", [this]()
	{
		FReferenceConverterTest TestIn;
		TObjectPtr<UReferenceConverterTestObject> TestOther = NewObject<UReferenceConverterTestObject>();
		TestIn.TestObjectPath = TestOther;

		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestObjectPath");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestObjectPath);
		TestObjectRefData(OutJson->AsObject(), TestOther->GetPathName());

		FReferenceConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestObjectPath));
		TestEqual("RefCheck", TestIn.TestObjectPath, TestOut.TestObjectPath);
	});

	It("Converts SoftPtr property to reference schema", [this]()
	{
		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestObjectSoft");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Prop);
		TestObjectRefSchema(Schema, UReferenceConverterTestObject::StaticClass(), false);
	});

	It("Converts SoftPtr property to and from JSON", [this]()
	{
		FReferenceConverterTest TestIn;
		TObjectPtr<UReferenceConverterTestObject> TestOther = NewObject<UReferenceConverterTestObject>();
		TestIn.TestObjectSoft = TestOther;

		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestObjectSoft");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestObjectSoft);
		TestObjectRefData(OutJson->AsObject(), TestOther->GetPathName());

		FReferenceConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestObjectSoft));
		TestEqual("RefCheck", TestIn.TestObjectSoft, TestOut.TestObjectSoft);
	});

	It("Converts WeakPtr property to reference schema", [this]()
	{
		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestObjectWeak");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Prop);
		TestObjectRefSchema(Schema, UReferenceConverterTestObject::StaticClass(), false);
	});

	It("Converts WeakPtr property to and from JSON", [this]()
	{
		FReferenceConverterTest TestIn;
		TObjectPtr<UReferenceConverterTestObject> TestOther = NewObject<UReferenceConverterTestObject>();
		TestIn.TestObjectWeak = TestOther;

		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestObjectWeak");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestObjectWeak);
		TestObjectRefData(OutJson->AsObject(), TestOther->GetPathName());

		FReferenceConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestObjectWeak));
		TestEqual("RefCheck", TestIn.TestObjectWeak, TestOut.TestObjectWeak);
	});

	It("Converts UClass* property to reference schema", [this]()
	{
		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestClass");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Prop);
		TestObjectRefSchema(Schema, UReferenceConverterTestObject::StaticClass(), true);
	});

	It("Converts UClass* property to and from JSON", [this]()
	{
		FReferenceConverterTest TestIn;
		TestIn.TestClass = UReferenceConverterTestObject::StaticClass();

		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestClass");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestClass);
		TestObjectRefData(OutJson->AsObject(), UReferenceConverterTestObject::StaticClass()->GetPathName());

		FReferenceConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestClass));
		TestEqual("RefCheck", TestIn.TestClass, TestOut.TestClass);
	});

	It("Converts SoftClassPath property to reference schema", [this]()
	{
		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestClassPath");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Prop);
		TestObjectRefSchema(Schema, UObject::StaticClass(), true);
	});

	It("Converts SoftClassPath property to and from JSON", [this]()
	{
		FReferenceConverterTest TestIn;
		TestIn.TestClassPath = UReferenceConverterTestObject::StaticClass();

		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestClassPath");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestClassPath);
		TestObjectRefData(OutJson->AsObject(), UReferenceConverterTestObject::StaticClass()->GetPathName());

		FReferenceConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestClassPath));
		TestEqual("RefCheck", TestIn.TestClassPath, TestOut.TestClassPath);
	});

	It("Imports from old string style path", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> TestOther = NewObject<UReferenceConverterTestObject>();
		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestObject");

		FReferenceConverterTest TestOut;
		TSharedPtr<FJsonValue> OutJson = MakeShared<TJsonValueString<TCHAR>>(TestOther->GetPathName());
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestObject));
		TestEqual("RefCheck", TestOut.TestObject, TestOther);
	});

	It("Converts bracket-named UObject* to and from JSON", [this]()
	{
		// Systems like UMG auto-generate bracket-prefixed names for non-variable widgets
		// (e.g. "[Overlay] Camera Mode Content"). ImportText_Direct truncates at '['
		// because FPropertyHelpers::ReadToken defines valid chars as a-zA-Z0-9_-+./:
		// The ReferenceConverter fix bypasses ImportText_Direct by using
		// FSoftObjectPath::ResolveObject() + SetObjectPropertyValue() directly.

		UObject* Outer = NewObject<UReferenceConverterTestObject>(
			GetTransientPackage(), TEXT("BracketNameTestOuter"));

		UReferenceConverterTestObject* BracketObj = NewObject<UReferenceConverterTestObject>(
			Outer, FName("[Overlay] Test Content"));

		FReferenceConverterTest TestIn;
		TestIn.TestObject = BracketObj;

		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestObject");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestObject);
		TestObjectRefData(OutJson->AsObject(), BracketObj->GetPathName());

		FReferenceConverterTest TestOut;
		TestTrue("Bracket name round-trips through JSON",
			JsonDataToProperty(OutJson, Property, &TestOut.TestObject));
		TestEqual("Resolved to same object", TestIn.TestObject, TestOut.TestObject);
	});

	It("Resolves bracket-named object via ExecuteTool", [this]()
	{
		UObject* Outer = NewObject<UReferenceConverterTestObject>(
			GetTransientPackage(), TEXT("BracketNameTestOuter2"));

		UReferenceConverterTestObject* BracketObj = NewObject<UReferenceConverterTestObject>(
			Outer, FName("[TestClass] My Content"));

		TObjectPtr<UReferenceConverterTestObject> TestObject = NewObject<UReferenceConverterTestObject>();
		UE::ToolsetRegistry::FJsonValueOrError Result =
			UE::ToolsetRegistry::TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestObjectParam"),
				FString::Printf(TEXT(R"({"testObject": {"refPath": "%s"}})"),
					*BracketObj->GetPathName()));

		TestFalse("Bracket-named refPath should resolve without error", Result.HasError());
	});

	It("Resolves space-named object via JSON round-trip", [this]()
	{
		// Control test: spaces without brackets should also work
		UObject* Outer = NewObject<UReferenceConverterTestObject>(
			GetTransientPackage(), TEXT("SpaceNameTestOuter"));

		UReferenceConverterTestObject* SpaceObj = NewObject<UReferenceConverterTestObject>(
			Outer, FName("Camera Mode Content"));

		FReferenceConverterTest TestIn;
		TestIn.TestObject = SpaceObj;

		UStruct* Struct = FReferenceConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestObject");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestObject);
		TestObjectRefData(OutJson->AsObject(), SpaceObj->GetPathName());

		FReferenceConverterTest TestOut;
		TestTrue("Space name round-trips through JSON",
			JsonDataToProperty(OutJson, Property, &TestOut.TestObject));
		TestEqual("Resolved to same object", TestIn.TestObject, TestOut.TestObject);
	});

	It("Creates instanced sub-object from class refPath", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> Outer = NewObject<UReferenceConverterTestObject>();
		FProperty* Property = UReferenceConverterTestObject::StaticClass()->FindPropertyByName(TEXT("TestInstancedObject"));
		void* PropPtr = Property->ContainerPtrToValuePtr<void>(Outer.Get());

		TSharedPtr<FJsonObject> JsonRef = MakeShared<FJsonObject>();
		JsonRef->SetStringField(TEXT("refPath"), UReferenceConverterTestObject::StaticClass()->GetPathName());
		TestTrue("Import", JsonDataToProperty(MakeShared<FJsonValueObject>(JsonRef), Property, PropPtr, Outer.Get()));

		UObject* Created = CastField<FObjectProperty>(Property)->GetObjectPropertyValue(PropPtr);
		if (!TestNotNull("Instance created", Created)) return;
		TestTrue("Correct class", Created->IsA(UReferenceConverterTestObject::StaticClass()));
		TestTrue("Outer is correct", Created->GetOuter() == Outer.Get());
	});

	It("Reuses existing instanced object when class matches", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> Outer = NewObject<UReferenceConverterTestObject>();
		FProperty* Property = UReferenceConverterTestObject::StaticClass()->FindPropertyByName(TEXT("TestInstancedObject"));
		void* PropPtr = Property->ContainerPtrToValuePtr<void>(Outer.Get());

		TSharedPtr<FJsonObject> JsonRef = MakeShared<FJsonObject>();
		JsonRef->SetStringField(TEXT("refPath"), UReferenceConverterTestObject::StaticClass()->GetPathName());

		TestTrue("First import", JsonDataToProperty(MakeShared<FJsonValueObject>(JsonRef), Property, PropPtr, Outer.Get()));
		UObject* First = CastField<FObjectProperty>(Property)->GetObjectPropertyValue(PropPtr);
		TestTrue("Second import", JsonDataToProperty(MakeShared<FJsonValueObject>(JsonRef), Property, PropPtr, Outer.Get()));
		UObject* Second = CastField<FObjectProperty>(Property)->GetObjectPropertyValue(PropPtr);
		TestEqual("Same instance reused", First, Second);
	});

	It("Does not create instance for invalid class path", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> Outer = NewObject<UReferenceConverterTestObject>();
		FProperty* Property = UReferenceConverterTestObject::StaticClass()->FindPropertyByName(TEXT("TestInstancedObject"));
		void* PropPtr = Property->ContainerPtrToValuePtr<void>(Outer.Get());

		TSharedPtr<FJsonObject> JsonRef = MakeShared<FJsonObject>();
		JsonRef->SetStringField(TEXT("refPath"), TEXT("/Invalid/Path/DoesNotExist"));
		// CaptureErrorsIn sets up a stack frame so RaiseScriptError is observable; without it
		// JsonDataToProperty's RaiseScriptError calls are silent.
		UE::ToolsetRegistry::FToolCallExceptionHandler Handler;
		bool ImportResult = true;
		Handler.CaptureErrorsIn([&]()
		{
			ImportResult = JsonDataToProperty(MakeShared<FJsonValueObject>(JsonRef), Property, PropPtr, Outer.Get());
		});
		TestFalse("Import reports failure", ImportResult);
		TestNull("No instance created", CastField<FObjectProperty>(Property)->GetObjectPropertyValue(PropPtr));
		TestTrue("Raises a validation error",
			Handler.GetException().Contains(TEXT("is not a valid object path"))
			|| Handler.GetException().Contains(TEXT("is not a valid subclass of")));
	});

	It("Does not create instance for wrong class type", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> Outer = NewObject<UReferenceConverterTestObject>();
		FProperty* Property = UReferenceConverterTestObject::StaticClass()->FindPropertyByName(TEXT("TestInstancedObject"));
		void* PropPtr = Property->ContainerPtrToValuePtr<void>(Outer.Get());

		// UObject is not a subclass of UReferenceConverterTestObject
		TSharedPtr<FJsonObject> JsonRef = MakeShared<FJsonObject>();
		JsonRef->SetStringField(TEXT("refPath"), UObject::StaticClass()->GetPathName());
		UE::ToolsetRegistry::FToolCallExceptionHandler Handler;
		bool ImportResult = true;
		Handler.CaptureErrorsIn([&]()
		{
			ImportResult = JsonDataToProperty(MakeShared<FJsonValueObject>(JsonRef), Property, PropPtr, Outer.Get());
		});
		TestFalse("Import reports failure", ImportResult);
		TestNull("No instance created", CastField<FObjectProperty>(Property)->GetObjectPropertyValue(PropPtr));
		TestTrue("Raises subclass validation error",
			Handler.GetException().Contains(TEXT("is not a valid subclass of")));
	});

	It("Rejects foreign object for instanced property", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> Outer = NewObject<UReferenceConverterTestObject>();
		// Foreign has a different outer (transient package), so it is not a subobject of Outer.
		TObjectPtr<UReferenceConverterTestObject> Foreign = NewObject<UReferenceConverterTestObject>();

		FProperty* Property = UReferenceConverterTestObject::StaticClass()->FindPropertyByName(TEXT("TestInstancedObject"));
		void* PropPtr = Property->ContainerPtrToValuePtr<void>(Outer.Get());

		TSharedPtr<FJsonObject> JsonRef = MakeShared<FJsonObject>();
		JsonRef->SetStringField(TEXT("refPath"), Foreign->GetPathName());
		UE::ToolsetRegistry::FToolCallExceptionHandler Handler;
		bool ImportResult = true;
		Handler.CaptureErrorsIn([&]()
		{
			ImportResult = JsonDataToProperty(MakeShared<FJsonValueObject>(JsonRef), Property, PropPtr, Outer.Get());
		});
		TestFalse("Import reports failure", ImportResult);
		TestNull("Foreign object not assigned", CastField<FObjectProperty>(Property)->GetObjectPropertyValue(PropPtr));
		TestTrue("Raises subobject validation error",
			Handler.GetException().Contains(TEXT("is not a subobject of the property owner")));
	});

	It("Round-trips instanced sub-object via subobject path", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> Outer = NewObject<UReferenceConverterTestObject>();
		TObjectPtr<UReferenceConverterTestObject> Sub = NewObject<UReferenceConverterTestObject>(Outer.Get());
		Outer->TestInstancedObject = Sub;

		FProperty* Property = UReferenceConverterTestObject::StaticClass()->FindPropertyByName(TEXT("TestInstancedObject"));
		void* PropPtr = Property->ContainerPtrToValuePtr<void>(Outer.Get());

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, PropPtr);

		Outer->TestInstancedObject = nullptr;
		TestTrue("Import", JsonDataToProperty(OutJson, Property, PropPtr, Outer.Get()));

		TestEqual("Same object", Outer->TestInstancedObject.Get(), Sub.Get());
	});

	It("Round-trips instanced array element via subobject path", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> Outer = NewObject<UReferenceConverterTestObject>();
		TObjectPtr<UReferenceConverterTestObject> Sub = NewObject<UReferenceConverterTestObject>(Outer.Get());
		Outer->TestInstancedArray.Add(Sub);

		FProperty* Property = UReferenceConverterTestObject::StaticClass()->FindPropertyByName(TEXT("TestInstancedArray"));
		void* PropPtr = Property->ContainerPtrToValuePtr<void>(Outer.Get());

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, PropPtr);

		Outer->TestInstancedArray.Empty();
		TestTrue("Import", JsonDataToProperty(OutJson, Property, PropPtr, Outer.Get()));

		if (!TestEqual("Array size", Outer->TestInstancedArray.Num(), 1)) return;
		TestEqual("Same object", Outer->TestInstancedArray[0].Get(), Sub.Get());
	});

	It("Creates instanced array element from class path with correct outer", [this]()
	{
		TObjectPtr<UReferenceConverterTestObject> Outer = NewObject<UReferenceConverterTestObject>();
		FProperty* Property = UReferenceConverterTestObject::StaticClass()->FindPropertyByName(TEXT("TestInstancedArray"));
		void* PropPtr = Property->ContainerPtrToValuePtr<void>(Outer.Get());

		TSharedPtr<FJsonObject> ElemJson = MakeShared<FJsonObject>();
		ElemJson->SetStringField(TEXT("refPath"), UReferenceConverterTestObject::StaticClass()->GetPathName());
		TArray<TSharedPtr<FJsonValue>> ArrayValues;
		ArrayValues.Add(MakeShared<FJsonValueObject>(ElemJson));
		TestTrue("Import", JsonDataToProperty(MakeShared<FJsonValueArray>(ArrayValues), Property, PropPtr, Outer.Get()));

		if (!TestEqual("Array size", Outer->TestInstancedArray.Num(), 1)) return;
		UReferenceConverterTestObject* Created = Outer->TestInstancedArray[0].Get();
		if (!TestNotNull("Element created", Created)) return;
		TestTrue("Correct class", Created->IsA(UReferenceConverterTestObject::StaticClass()));
		TestEqual("Correct outer", Created->GetOuter(), static_cast<UObject*>(Outer.Get()));
	});

	It("Can read null default for object reference", [this]()
	{
		UFunction* FunctionPtr = UReferenceConverterTestObject::StaticClass()->FindFunctionByName("TestDefault");
		const TSharedPtr<FJsonObject> Schema = StructToJsonSchema(FunctionPtr);
		if (!TestTrue("Schema", Schema.IsValid())) return;

		const TSharedPtr<FJsonObject>* InputSchema = nullptr;
		if (!(TestTrue("Input", Schema->TryGetObjectField(TEXT("inputSchema"), InputSchema)) &&
			InputSchema))
		{
			return;
		}

		TestFalse("NoRequired", InputSchema->Get()->HasField(TEXT("required")));

		const TSharedPtr<FJsonObject>* Properties = nullptr;
		if (!(TestTrue("Properties", InputSchema->Get()->TryGetObjectField(TEXT("properties"), Properties)) &&
			Properties))
		{
			return;
		}
		
		const TSharedPtr<FJsonObject>* TestObject = nullptr;
		if (!(TestTrue("testObject", Properties->Get()->TryGetObjectField(TEXT("testObject"), TestObject)) &&
			TestObject))
		{
			return;
		}
		TestTrue("Default", TestObject->Get()->GetField(TEXT("default"), EJson::Null)->IsNull());
	});
}

#endif
