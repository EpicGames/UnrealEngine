// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"

#include "ReferenceConverterTest.generated.h"


UCLASS(BlueprintType)
class UReferenceConverterTestObject : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(Instanced)
	TObjectPtr<UReferenceConverterTestObject> TestInstancedObject;

	UPROPERTY(Instanced)
	TArray<TObjectPtr<UReferenceConverterTestObject>> TestInstancedArray;

	UFUNCTION(BlueprintCallable, Category="ReferenceConverterTest")
	void TestOptionalObject(UObject* TestObject = nullptr) {}

	UFUNCTION(BlueprintCallable, Category="ReferenceConverterTest")
	void TestObjectParam(UObject* TestObject) {}

	UFUNCTION(BlueprintCallable, Category="ReferenceConverterTest")
	void TestSpecObjectParam(UReferenceConverterTestObject* TestObject) {}

	UFUNCTION(BlueprintCallable, Category="ReferenceConverterTest")
	void TestClassParam(UClass* TestClass) {}

	UFUNCTION(BlueprintCallable, Category="ReferenceConverterTest")
	void TestSubClassParam(TSubclassOf<UReferenceConverterTestObject> TestClass) {}

	UFUNCTION(BlueprintCallable, Category="ReferenceConverterTest")
	void TestSoftObjectParam(TSoftObjectPtr<UReferenceConverterTestObject> TestObject) {}

	UFUNCTION(BlueprintCallable, Category="ReferenceConverterTest")
	void TestSoftClassParam(TSoftClassPtr<UReferenceConverterTestObject> TestClass) {}

	UFUNCTION(BlueprintCallable, Category="ReferenceConverterTest")
	void TestSoftClassPathParam(FSoftClassPath TestClassPath) {}

	UFUNCTION(BlueprintCallable, Category="ReferenceConverterTest")
	void TestDefault(UObject* TestObject = nullptr) {}
};

USTRUCT(BlueprintType)
struct FReferenceConverterTest
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TObjectPtr<UReferenceConverterTestObject> TestObject;

	UPROPERTY()
	FSoftObjectPath TestObjectPath;

	UPROPERTY()
	TSoftObjectPtr<UReferenceConverterTestObject> TestObjectSoft;

	UPROPERTY()
	TWeakObjectPtr<UReferenceConverterTestObject> TestObjectWeak;

	UPROPERTY()
	TSubclassOf<UReferenceConverterTestObject> TestClass;

	UPROPERTY()
	FSoftClassPath TestClassPath;
};
