// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/RunOnMainThread.h"

#include "Async/Async.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformTLS.h"
#include "Misc/AutomationTest.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

#include "ToolsetRegistryTestFlags.h"
#include "Tests/UntilFutureCommand.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FRunOnMainThreadTest, "AI.ToolsetRegistry.RunOnMainThread",
	ToolsetRegistryTest::Flags)

	// Ensure the specified thread ID is set and matches the game thread ID.
	void TestEqualsGameThreadId(TOptional<uint32>& MaybeThreadId)
	{
		if (!TestTrue(TEXT("Thread ID is set"), MaybeThreadId.IsSet())) return;
		TestEqual(TEXT("Thread ID is game thread"), *MaybeThreadId, GameThreadId);
	}

	// Ensure the specified future has the expected result.
	void TestFutureHasExpectedResult(TFuture<TUniquePtr<uint32>>& Future)
	{
		if (!TestTrue(TEXT("Future is valid"), Future.IsValid())) return;
		TestIsExpectedResult(Future.Consume());
	}

	// Ensure the specified value is the expected result.
	void TestIsExpectedResult(TUniquePtr<uint32>&& Result)
	{
		if (!TestTrue(TEXT("Result is set"), Result.IsValid())) return;
		TestEqual(
			TEXT("Future value matches"), *Result,
			*FRunOnMainThreadTest::CreateExpectedResult());
	}

private:
	uint32 GameThreadId;

private:
	static TSharedPtr<TOptional<uint32>> CreateMaybeThreadId()
	{
		return MakeShared<TOptional<uint32>>();
	}

	static TUniquePtr<uint32> CreateExpectedResult()
	{
		return MakeUnique<uint32>(42);
	}

END_DEFINE_SPEC(FRunOnMainThreadTest)

void FRunOnMainThreadTest::Define()
{
	using UE::ToolsetRegistry::Internal::RunOnMainThread;
	using UE::ToolsetRegistry::Internal::FUntilFutureCommand;

	BeforeEach(
		[this]() -> void
		{
			GameThreadId = FPlatformTLS::GetCurrentThreadId();
		});

	It(TEXT(
		"Should run a function that returns nothing on the main thread when called from the main "
		"thread"),
		[this]() -> void
	{
		auto MaybeThreadId = CreateMaybeThreadId();
		RunOnMainThread(
			[MaybeThreadId]() -> void
			{
				MaybeThreadId->Emplace(FPlatformTLS::GetCurrentThreadId());
			}).Wait();
		TestEqualsGameThreadId(*MaybeThreadId);
	});

	It(TEXT(
		"Should run a function that returns nothing on the main thread when called from another "	
		"thread."),
		[this]() -> void
	{
		auto MaybeThreadId = CreateMaybeThreadId();
		auto Result = Async(
			EAsyncExecution::Thread,
			[MaybeThreadId]() -> void
			{
				RunOnMainThread(
					[MaybeThreadId]() -> void
					{
						MaybeThreadId->Emplace(FPlatformTLS::GetCurrentThreadId());
					}).Wait();
			});
		AddCommand(
			FUntilFutureCommand::Create<void>(
				MoveTemp(Result),
				[this, MaybeThreadId]() -> void
				{
					TestEqualsGameThreadId(*MaybeThreadId);
				}));
	});

	It(TEXT(
		"Should run a function that returns a value on the main thread when called from the main "
		"thread"),
		[this]() -> void
	{
		auto MaybeThreadId = CreateMaybeThreadId();
		// This returns TUniquePtr<uint32> to ensure that the return value is moved.
		auto Result = RunOnMainThread(
			[MaybeThreadId]() -> TUniquePtr<uint32>
			{
				MaybeThreadId->Emplace(FPlatformTLS::GetCurrentThreadId());
				return FRunOnMainThreadTest::CreateExpectedResult();
			});
		TestEqualsGameThreadId(*MaybeThreadId);
		TestFutureHasExpectedResult(Result);
	});

	It(TEXT(
		"Should run a function that returns a value on the main thread when called from another "
		"thread."),
		[this]() -> void
	{
		auto MaybeThreadId = CreateMaybeThreadId();
		// This returns TUniquePtr<uint32> to ensure that the return value is moved.
		auto Result = Async(
			EAsyncExecution::Thread,
			[MaybeThreadId]() -> TUniquePtr<uint32>
			{
				return 
					RunOnMainThread(
						[MaybeThreadId]() -> TUniquePtr<uint32>
						{
							MaybeThreadId->Emplace(FPlatformTLS::GetCurrentThreadId());
							return FRunOnMainThreadTest::CreateExpectedResult(); 
						}).Consume();
			});
		AddCommand(
			FUntilFutureCommand::Create<TUniquePtr<uint32>>(
				MoveTemp(Result),
				[this, MaybeThreadId](TUniquePtr<uint32>&& ResultValue) -> void
				{
					TestEqualsGameThreadId(*MaybeThreadId);
					TestIsExpectedResult(MoveTemp(ResultValue));
				}));
		
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS