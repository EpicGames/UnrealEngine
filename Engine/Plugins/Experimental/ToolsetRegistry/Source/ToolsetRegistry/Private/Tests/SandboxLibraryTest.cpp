// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/SandboxLibrary.h"
#include "Tests/ToolsetRegistryTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::ToolsetRegistry;

// -----------------------------------------------------------------------------
// SandboxLibrary tests
// -----------------------------------------------------------------------------

BEGIN_DEFINE_SPEC(
	FSandboxLibrarySpec,
	"AI.ToolsetRegistry.Sandbox.Library",
	ToolsetRegistryTest::Flags)
END_DEFINE_SPEC(FSandboxLibrarySpec)

void FSandboxLibrarySpec::Define()
{
	// ToolsetRegistry depends on FileSandboxCore, so the module is always available.

	Describe(TEXT("When no sandbox is active"), [this]
	{
		It(TEXT("IsActive should return false"), [this]
		{
			if (FGlobalSandbox::IsActive()) return; // skip if a sandbox is unexpectedly active
			TestFalse(TEXT("No active sandbox"), FGlobalSandbox::IsActive());
		});

		It(TEXT("GetActiveName should return empty"), [this]
		{
			if (FGlobalSandbox::IsActive()) return;
			TestTrue(TEXT("Name is empty"), FGlobalSandbox::GetActiveName().IsEmpty());
		});

		It(TEXT("GetChanges should return empty array"), [this]
		{
			if (FGlobalSandbox::IsActive()) return;
			TestEqual(TEXT("No changes"), FGlobalSandbox::GetChanges().Num(), 0);
		});

		It(TEXT("Leave should return true (no-op)"), [this]
		{
			if (FGlobalSandbox::IsActive()) return;
			TestTrue(TEXT("Leave succeeds as no-op"), FGlobalSandbox::Leave());
		});
	});

	Describe(TEXT("Sandbox lifecycle"), [this]
	{
		const FString TestSandboxName = TEXT("ToolsetRegistryTest_SandboxLibrary");

		BeforeEach([this, TestSandboxName]
		{
			// Ensure no sandbox is active before each test.
			FGlobalSandbox::Leave();
		});

		AfterEach([this, TestSandboxName]
		{
			// Always clean up: discard any sandbox changes, then leave.
			if (FGlobalSandbox::IsActive())
			{
				FGlobalSandbox::Discard();
			}
			FGlobalSandbox::Leave();
		});

		It(TEXT("Enter creates a new sandbox and IsActive returns true"), [this, TestSandboxName]
		{
			TestTrue(TEXT("Enter succeeded"), FGlobalSandbox::Enter(TestSandboxName, TEXT("")));
			TestTrue(TEXT("Sandbox is now active"), FGlobalSandbox::IsActive());
			TestEqual(TEXT("Active sandbox has expected name"), FGlobalSandbox::GetActiveName(), TestSandboxName);
		});

		It(TEXT("Enter with the same name is a no-op when already active"), [this, TestSandboxName]
		{
			TestTrue(TEXT("First Enter succeeded"), FGlobalSandbox::Enter(TestSandboxName, TEXT("")));
			TestTrue(TEXT("Second Enter succeeded"), FGlobalSandbox::Enter(TestSandboxName, TEXT("")));
			TestTrue(TEXT("Sandbox still active"), FGlobalSandbox::IsActive());
		});

		It(TEXT("Leave deactivates the sandbox"), [this, TestSandboxName]
		{
			TestTrue(TEXT("Enter succeeded"), FGlobalSandbox::Enter(TestSandboxName, TEXT("")));
			TestTrue(TEXT("Leave succeeded"), FGlobalSandbox::Leave());
			TestFalse(TEXT("Sandbox no longer active"), FGlobalSandbox::IsActive());
		});

		It(TEXT("GetChanges returns empty array for a fresh sandbox"), [this, TestSandboxName]
		{
			TestTrue(TEXT("Enter succeeded"), FGlobalSandbox::Enter(TestSandboxName, TEXT("")));
			TestEqual(TEXT("No changes in fresh sandbox"), FGlobalSandbox::GetChanges().Num(), 0);
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
