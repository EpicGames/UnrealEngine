// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Templates/SharedPointer.h"
#include "ToolsetRegistry/ToolCallAsyncResultImage.h"
#include "ToolsetRegistry/ToolsetImage.h"
#include "Tests/ToolCallAsyncResultTest.h"

#if WITH_DEV_AUTOMATION_TESTS

UE_TOOLSET_REGISTRY_TOOL_CALL_ASYNC_RESULT_WITH_VALUE_SPEC(
	Image, UToolCallAsyncResultImage, FToolsetImage,
	[]() -> FToolsetImage
	{
		FToolsetImage Image;
		Image.MimeType = TEXT("image/png");
		Image.Data = TEXT("abc123");
		return Image;
	},
	[](const FToolsetImage& ExpectedValue) -> TSharedPtr<FJsonValue>
	{
		TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		JsonObject->SetStringField(TEXT("mimeType"), ExpectedValue.MimeType);
		JsonObject->SetStringField(TEXT("data"), ExpectedValue.Data);
		return MakeShared<FJsonValueObject>(JsonObject);
	})

#endif  // WITH_DEV_AUTOMATION_TESTS
