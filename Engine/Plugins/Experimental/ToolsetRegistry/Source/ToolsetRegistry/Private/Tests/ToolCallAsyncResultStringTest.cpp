// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolCallAsyncResultString.h"
#include "Tests/ToolCallAsyncResultTest.h"

#if WITH_DEV_AUTOMATION_TESTS

UE_TOOLSET_REGISTRY_TOOL_CALL_ASYNC_RESULT_WITH_VALUE_SPEC(
	String, UToolCallAsyncResultString, FString,
	[]() -> FString { return TEXT("Done"); },
	[](const FString& ExpectedValue) -> TSharedPtr<FJsonValue> {
		return MakeShared<FJsonValueString>(ExpectedValue);
	})

#endif  // WITH_DEV_AUTOMATION_TESTS	