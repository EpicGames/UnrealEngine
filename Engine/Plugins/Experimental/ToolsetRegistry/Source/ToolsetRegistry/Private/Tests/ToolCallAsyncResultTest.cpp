// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ToolCallAsyncResultTest.h"

#include "Async/Async.h"
#include "Dom/JsonObject.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationTest.h"
#include "Misc/Optional.h"
#include "HAL/Event.h"
#include "Templates/FunctionFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"

#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/ToolCallAsyncResult.h"
#include "ToolsetRegistry/ToolsetJson.h"

#if WITH_DEV_AUTOMATION_TESTS

// On destruction or when calling Validate(), validates the completion of a UToolCallAsyncResult.
class FToolCallAsyncResultCompletionValidator
{
public:
	enum class EResultHandlerMode
	{
		// Enable the result handler on construction.
		Enabled,
		// Delay construction of the result handler until it's accessed.
		Deferred,
	};

public:
	FToolCallAsyncResultCompletionValidator(
		FAutomationTestBase& TestCaseIn,
		TObjectPtr<UToolCallAsyncResult> ResultIn,
		EResultHandlerMode ResultHandlerMode = EResultHandlerMode::Enabled) :
		TestCase(TestCaseIn),
		Result(ResultIn)
	{
		if (ResultHandlerMode == EResultHandlerMode::Enabled) EnsureResultHandler();
	}

	virtual ~FToolCallAsyncResultCompletionValidator()
	{
		if (!bValidated) Validate();
	}

	// Prevent copy.
	FToolCallAsyncResultCompletionValidator(
		const FToolCallAsyncResultCompletionValidator&) = delete;
	FToolCallAsyncResultCompletionValidator& operator=(
		const FToolCallAsyncResultCompletionValidator&) = delete;

	// Check all expectations.
	void Validate()
	{
		check(Result);
		if (bShouldComplete)
		{
			// Verify a complete result.
			TestCase.TestTrue(TEXT("Result is complete"), Result->bIsComplete);
			if (ExpectedError.IsEmpty())
			{
				EnsureResultCompletedWithValue(GetExpectedJsonValueString());
			}
			else
			{
				EnsureResultCompletedWithError(ExpectedError, ExpectedError);
			}
		}
		else
		{
			EnsureResultIsNotComplete();
		}
		bValidated = true;
	}

	FToolCallAsyncResultCompletionValidator& ExpectShouldComplete(
		bool bShouldCompleteIn)
	{
		bShouldComplete = bShouldCompleteIn;
		return *this;
	}
	
	FToolCallAsyncResultCompletionValidator& ExpectError(
		const FString& Error)
	{
		ExpectedError = Error;
		return *this;
	}

	// A value must be expected to validate a result returns either a value or no value.
	FToolCallAsyncResultCompletionValidator& ExpectJsonValue(
		TSharedPtr<FJsonValue> JsonValue)
	{
		ExpectedJsonValue = JsonValue;
		return *this;
	}

	// Ensure a result handler is created and subscribe to the result.
	void EnsureResultHandler()
	{
		if (!ResultHandler)
		{
			ResultHandler = UToolCallAsyncResultFutureHandler::Create(Result.Get());
			ResultHandlerFuture.Emplace(ResultHandler->GetValueAsJson());
		}
	}

	TStrongObjectPtr<UToolCallAsyncResultFutureHandler> GetResultHandler()
	{
		EnsureResultHandler();
		return ResultHandler;
	}

	TFuture<UE::ToolsetRegistry::FJsonValueOrError>& GetResultHandlerFuture()
	{
		EnsureResultHandler();
		return ResultHandlerFuture.GetValue();
	}

	FAutomationTestBase& GetTestCase() { return TestCase; }

	FString GetExpectedJsonValueString() const
	{
		check(ExpectedJsonValue);
		return UE::ToolsetRegistry::Internal::JsonToString(ExpectedJsonValue.ToSharedRef());
	}

	void EnsureResultCompletedWithValue(const FString& ExpectedJsonValueString)
	{
		// With no error.
		TestCase.TestEqual(TEXT("Result has no error"), Result->Error, TEXT(""));
		// NOTE: The JSON value is tested indirectly by converting to a string.
		if (ExpectedJsonValue)
		{
			TestCase.TestTrue(TEXT("Result has JSON value"),
				Result->GetValueAsJson().IsValid());
			TestCase.TestEqual(
				TEXT("Result matches expected JSON string"),
				Result->GetValueAsJsonString(), ExpectedJsonValueString);
		}

		// Verify the OnCompleted event fired and was handled by the handler.
		if (ResultHandlerFuture.IsSet() && 
			TestCase.TestTrue(
				TEXT("Handler was signaled"), ResultHandlerFuture->IsReady()) &&
			TestCase.TestTrue(
				TEXT("Handler result has value"), ResultHandlerFuture->Get().HasValue()))
		{
			TestCase.TestEqual(
				TEXT("Handler result value"),
				UE::ToolsetRegistry::Internal::JsonToString(
					ResultHandlerFuture->Get().GetValue().ToSharedRef()),
				ExpectedJsonValueString);
		}
	}

	void EnsureResultCompletedWithError(
		const FString& InExpectedError, const FString ExpectedHandlerError)
	{
		// That reports an error.
		TestCase.TestEqual(TEXT("Result has an error"), Result->Error, InExpectedError);
		if (ExpectedJsonValue)
		{
			TestCase.TestFalse(
				TEXT("Result has no JSON value"), Result->GetValueAsJson().IsValid());
		}
		TestCase.TestEqual(
			TEXT("Result has no JSON string"), Result->GetValueAsJsonString(), TEXT(""));

		// Verify the OnCompleted event fired and was handled by the handler.
		if (ResultHandlerFuture.IsSet() && 
			TestCase.TestTrue(TEXT("Handler was signaled"), ResultHandlerFuture->IsReady()) &&
			TestCase.TestTrue(
				TEXT("Handler result has error"),
				ResultHandlerFuture->Get().HasError()))
		{
			TestCase.TestEqual(
				TEXT("Handler result error"),
				ResultHandlerFuture->Get().GetError(), ExpectedHandlerError);
		}
	}

	void EnsureResultIsNotComplete()
	{
		// Verify a result that isn't complete.
		TestCase.TestFalse(TEXT("Result is not complete"), Result->bIsComplete);
		TestCase.TestEqual(TEXT("Result has no error"), Result->Error, TEXT(""));
		if (ExpectedJsonValue)
		{
			TestCase.TestFalse(TEXT("Result has no JSON value"), Result->GetValueAsJson().IsValid());
		}
		TestCase.TestEqual(
			TEXT("Result has no JSON string"), Result->GetValueAsJsonString(), TEXT(""));

		// Verify the OnCompleted event has not fired.
		TestCase.TestFalse(
			TEXT("Result handler future is not complete"),
			ResultHandlerFuture.IsSet() && ResultHandlerFuture->IsReady());
	}

private:
	FAutomationTestBase& TestCase;
	TStrongObjectPtr<UToolCallAsyncResult> Result;
	TStrongObjectPtr<UToolCallAsyncResultFutureHandler> ResultHandler;
	TOptional<TFuture<UE::ToolsetRegistry::FJsonValueOrError>> ResultHandlerFuture;
	bool bShouldComplete = true;
	FString ExpectedError;
	TSharedPtr<FJsonValue> ExpectedJsonValue;
	bool bValidated = false;
};

namespace
{
	// Handle to a task started with ExecuteOnThread.
	template<typename ReturnT>
	class FExecuteOnThreadHandle
	{
	public:
		FExecuteOnThreadHandle(
			TSharedPtr<FEventRef> InRunTaskEvent,
			TFuture<ReturnT>&& InReturnValueFuture) :
			ReturnValueFuture(MoveTemp(InReturnValueFuture)),
			RunTaskEvent(InRunTaskEvent)
		{
		}

		// Prevent copy.
		FExecuteOnThreadHandle(const FExecuteOnThreadHandle&) = delete;
		FExecuteOnThreadHandle& operator=(const FExecuteOnThreadHandle&) = delete;

		// Start the task associated with this handle.
		void Start() { RunTaskEvent->Get()->Trigger(); }

		// Get the return value of the task.
		ReturnT ReturnValue() const { return ReturnValueFuture.Get();  }

	private:
		TFuture<ReturnT> ReturnValueFuture;
		TSharedPtr<FEventRef> RunTaskEvent;
	};

	// Start a thread, wait until ExecuteOnThreadHandle::Start() is called and then run
	// Callable.
	template<typename CallableType>
	auto ExecuteOnThread(CallableType&& Callable) ->
		TSharedPtr<FExecuteOnThreadHandle<decltype(Forward<CallableType>(Callable)())>>
	{
		TSharedPtr<FEventRef> RunTaskEvent = MakeShared<FEventRef>();
		auto ReturnValueFuture = Async(
			EAsyncExecution::Thread,
			[RunTaskEvent, Callable = MoveTemp(Callable)]() -> auto
			{
				RunTaskEvent->Get()->Wait();
				return Callable();
			});
		return MakeShared<FExecuteOnThreadHandle<decltype(Forward<CallableType>(Callable)())>>(
			RunTaskEvent, MoveTemp(ReturnValueFuture));
	}

	// Modify an async result in a latent command and then validate its state.
	class FModifyAndValidateAsyncResultCommand
	{
	private:
		struct FState
		{
			TSharedPtr<FToolCallAsyncResultCompletionValidator> Validator;
			TFunction<bool()> ModifyResult;
			bool bHasModifiedResult = false;
		};

	public:
		static FUntilCommand* Create(
			TSharedPtr<FToolCallAsyncResultCompletionValidator> InValidator,
			TFunction<bool()>&& InModifyResult)
		{
			auto State = MakeShared<FState>();
			State->Validator = InValidator;
			State->ModifyResult = InModifyResult;
			return new FUntilCommand(
				[State]() -> bool
				{
					if (!State->bHasModifiedResult)
					{
						if (!State->ModifyResult()) return true;
						State->bHasModifiedResult = true;
					}

					// Wait for the async result to complete.
					if (!State->Validator->GetResultHandlerFuture().IsReady()) return false;

					State->Validator->Validate();
					return true;
				});
		}

		static FUntilCommand* Create(
			TSharedPtr<FToolCallAsyncResultCompletionValidator> Validator,
			TSharedPtr<FExecuteOnThreadHandle<bool>> ExecuteOnThreadHandle,
			FString ExecuteOnThreadTaskDescription)
		{
			return Create(
				Validator,
				[ExecuteOnThreadHandle, ExecuteOnThreadTaskDescription, Validator]() -> bool
				{
					ExecuteOnThreadHandle->Start();
					return Validator->GetTestCase().TestTrue(
						*ExecuteOnThreadTaskDescription, ExecuteOnThreadHandle->ReturnValue());
				});
		}
	};

}  // namespace

void FToolCallAsyncResultBaseSpec::Define()
{
	static const FString FakeError(TEXT("Failed"));

	// If we can't create a result, skip all tests.
	if (!CreateResult) return;

	BeforeEach([this]()
	{
		Result = CreateResult();
	});

	It(TEXT("Should not be complete on construction"), [this]()
	{
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectShouldComplete(false);
	});

	It(TEXT("Should not send completion if not complete"), [this]()
	{
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectShouldComplete(false);
		TestFalse(TEXT("Not complete"), Result->BroadcastOnCompletedIfComplete());
	});

	It(TEXT("Should complete when an error is set"), [this]()
	{
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectError(FakeError);
		TestTrue(TEXT("Set error returns true"), Result->SetError(FakeError));
	});

	It(TEXT("Should send completion with an error when already completed"), [this]()
	{
		Result->SetError(FakeError);
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectError(FakeError);
		TestTrue(TEXT("Is complete"), Result->bIsComplete);
	});

	It(TEXT("Should fail when an empty error is set"), [this]()
	{
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectShouldComplete(false);
		AddExpectedMessage(
			TEXT("Attempted to complete '.*' with an empty error!"),
			ELogVerbosity::Type::Error, EAutomationExpectedMessageFlags::Contains, 1, true);
		TestFalse(TEXT("Set an empty error returns false"), Result->SetError(TEXT("")));
	});

	It(TEXT("Should fail when an error is set more than once"), [this]()
	{
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectError(FakeError);
		Result->SetError(FakeError);
		AddExpectedMessage(
			TEXT("Unable to complete '.*' more than once"),
			ELogVerbosity::Type::Error, EAutomationExpectedMessageFlags::Contains, 1, true);
		TestFalse(TEXT("Set error twice returns false"), Result->SetError(FakeError));
	});

	It(TEXT("Should complete when an error is set from another thread"), [this]()
	{
		auto Validator =
			MakeShared<FToolCallAsyncResultCompletionValidator>(*this, Result);
		Validator->ExpectError(FakeError);

		AddCommand(
			FModifyAndValidateAsyncResultCommand::Create(
				Validator,
				ExecuteOnThread([this]() -> bool { return Result->SetError(FakeError); }),
				TEXT("Scheduled set error")));
	});

	It(TEXT("Should send completion with an error on broadcast from another thread"), [this]()
	{
		Result->SetError(FakeError);
		auto Validator =
			MakeShared<FToolCallAsyncResultCompletionValidator>(
				*this, Result,
				FToolCallAsyncResultCompletionValidator::EResultHandlerMode::Deferred);
		Validator->ExpectError(FakeError);

		AddCommand(
			FModifyAndValidateAsyncResultCommand::Create(
				Validator,
				ExecuteOnThread(
					[this]() -> bool
					{
						return Result->BroadcastOnCompletedIfComplete();
					}),
				TEXT("Schedule broadcast on completed")));
	});

	It(TEXT("Can unsubscribe from a result"), [this]()
	{
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectShouldComplete(false);
		Validator.Validate();
		Validator.GetResultHandler()->Unsubscribe();
		Result->SetError(FakeError);  // Should have no effect on the unsubscribed handler.

		Validator.EnsureResultCompletedWithError(
			FakeError, UToolCallAsyncResultFutureHandler::CanceledError);
	});

	// If a value setter and expected value are not set, skip value tests.
	if (!(SetResultValue && ExpectedJsonValue)) return;

	It(TEXT("Should complete when a value is set"), [this]()
	{
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectJsonValue(ExpectedJsonValue);
		TestTrue(TEXT("Successfully set value"), SetResultValue());
	});

	It(TEXT("Should send completion with a value when already completed"), [this]()
	{
		SetResultValue();
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectJsonValue(ExpectedJsonValue);
	});

	It(TEXT("Should fail when a value is set more than once"), [this]()
	{
		FToolCallAsyncResultCompletionValidator Validator(*this, Result);
		Validator.ExpectJsonValue(ExpectedJsonValue);
		SetResultValue();
		AddExpectedMessage(
			TEXT("Unable to complete '.*' more than once"),
			ELogVerbosity::Type::Error, EAutomationExpectedMessageFlags::Contains, 1, true);
		(void)TestFalse(TEXT("Setting the value twice should fail"), SetResultValue());
	});

	It(TEXT("Should complete when a value is set from another thread"), [this]()
	{
		auto Validator = MakeShared<
			FToolCallAsyncResultCompletionValidator>(*this, Result);
		Validator->ExpectJsonValue(ExpectedJsonValue);

		AddCommand(
			FModifyAndValidateAsyncResultCommand::Create(
				Validator,
				ExecuteOnThread([this]() -> bool { return SetResultValue(); }),
				TEXT("Schedule set value")));
	});

	It(TEXT("Should send completion with a value on broadcast from another thread"), [this]()
	{
		SetResultValue();
		auto Validator =
			MakeShared<FToolCallAsyncResultCompletionValidator>(
				*this, Result,
				FToolCallAsyncResultCompletionValidator::EResultHandlerMode::Deferred);
		Validator->ExpectJsonValue(ExpectedJsonValue);

		AddCommand(
			FModifyAndValidateAsyncResultCommand::Create(
				Validator,
				ExecuteOnThread(
					[this]() -> bool
					{
						return Result->BroadcastOnCompletedIfComplete();
					}),
				TEXT("Schedule broadcast on completed")));
	});

	It(TEXT("Should provide the value's JSON schema"), [this]()
	{
		using namespace UE::ToolsetRegistry::Internal;

		// This is a bit odd since it partially mirrors the implementation. However, the
		// alternative would be to force all value types to specify their expected schema.
		FProperty* Property = Result->GetClass()->FindPropertyByName(TEXT("Value"));
		if (!Property) return;  // No value, ignore

		TSharedPtr<FJsonObject> ExpectedSchema = ToolsetJson::PropertyToJsonSchema(Property);
		if (!TestTrue(TEXT("Value has schema"), ExpectedSchema.IsValid())) return;

		TestEqual(
			TEXT("Value schema"),
			JsonToString(UToolCallAsyncResult::GetValueJsonSchema(Result->GetClass())),
			JsonToString(ExpectedSchema.ToSharedRef()));
	});
}

// Test functionality of UToolCallAsyncResult where there is no way to successfully complete.
class FToolCallAsyncResultNoCompletionSpec :
	public FToolCallAsyncResultBaseSpec
{
public:
	using BaseSpec::BaseSpec;

	virtual void Define() override
	{
		SetupCreateResult<UToolCallAsyncResult>();
		BaseSpec::Define();
	}

	virtual FString GetBeautifiedTestName() const override
	{
		return TEXT("AI.ToolsetRegistry.ToolCallAsyncResult.NoCompletion");
	}
};

namespace
{
	FToolCallAsyncResultNoCompletionSpec FToolCallAsyncResultNoCompletionSpecInstance(
		TEXT("FToolCallAsyncResultNoCompletionSpec"));
}

BEGIN_DEFINE_SPEC(
	FToolCallAsyncResultTestProperty,
	"AI.ToolsetRegistry.ToolCallAsyncResult.Property",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolCallAsyncResultTestProperty)

void FToolCallAsyncResultTestProperty::Define()
{
	Describe(TEXT("MatchesProperty"), [this]()
	{
		It(TEXT("Should match an async result class"), [this]()
		{
			// NOTE: The additional null pointer checks in the following conditionals are to
			// silence false positive null pointer deferences flagged by MSVC's static analyzer.
			UFunction* Function = UToolCallAsyncResultReturnPropertyTest::FindTestFunction();
			if (!TestTrue(TEXT("Has test function"), Function != nullptr) || !Function) return;
			FProperty* ReturnProperty = Function->GetReturnProperty();
			if (!TestTrue(TEXT("Has return property"), ReturnProperty != nullptr)
				|| !ReturnProperty)
			{
				return;
			}
			UClass* ReturnClass = UToolCallAsyncResult::MatchesProperty(ReturnProperty);
			if (!TestTrue(TEXT("Is a class"), ReturnClass != nullptr) || !ReturnClass)
			{
				return;
			}
			TestTrue(
				TEXT("Returns async result class"),
				ReturnClass->IsChildOf<UToolCallAsyncResult>());
		});

		It(TEXT("Should not match a non-async result class"), [this]()
		{
			UFunction* Function = UToolCallAsyncResultReturnPropertyTest::FindTestFunction();
			if (!TestTrue(TEXT("Has test function"), Function != nullptr) || !Function) return;
			FProperty* Argument = Function->FindPropertyByName(TEXT("Subject"));
			if (!TestTrue(TEXT("Has argument"), Argument != nullptr) || !Argument) return;
			TestTrue(
				TEXT("Does not match non-async result property"),
				UToolCallAsyncResult::MatchesProperty(Argument) == nullptr);
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS