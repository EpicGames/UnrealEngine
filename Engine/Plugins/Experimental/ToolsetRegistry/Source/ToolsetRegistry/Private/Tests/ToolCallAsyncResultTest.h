// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Misc/AutomationTest.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "ToolsetRegistry/ToolCallAsyncResult.h"
#include "ToolsetRegistry/ToolCallAsyncResultFutureHandler.h"
#include "ToolsetRegistry/ToolCallAsyncResultString.h"
#include "ToolsetRegistry/ToolCallAsyncResultVoid.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

#include "ToolCallAsyncResultTest.generated.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(
	FToolCallAsyncResultBaseSpec,
	"AI.ToolsetRegistry.ToolCallAsyncResult.Base",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	using BaseSpec = FToolCallAsyncResultBaseSpec;

	// Called by test cases to set the result's value.
	// This *must* be set to execute the tests.
	TFunction<bool()> SetResultValue;
	// Queried by test cases to check the result's value.
	// This *must* be set to execute the tests.
	TSharedPtr<FJsonValue> ExpectedJsonValue;
	// Called by before each test case to create the result.
	TFunction<TObjectPtr<UToolCallAsyncResult>()> CreateResult;
	// Result operated on in each test.
	TObjectPtr<UToolCallAsyncResult> Result;

	// Set the CreateResult function to create an AsyncResultT instance.
	template<typename AsyncResultT>
	void SetupCreateResult()
	{
		CreateResult = [this]()
			{
				return TObjectPtr<UToolCallAsyncResult>(NewObject<AsyncResultT>());
			};
	}

	// Setup SetResultValue, ExpectedJsonValue and CreateResult.
	template<typename AsyncResultT, typename ValueT>
	void Setup(
		TFunction<TSharedPtr<FJsonValue>(const ValueT&)>& CreateExpectedJsonValue,
		TFunction<ValueT()>& InCreateExpectedValue)
	{
		SetupCreateResult<AsyncResultT>();

		SetResultValue =
			[this, CreateExpectedValue = InCreateExpectedValue]()
			{
				return CastChecked<AsyncResultT>(Result)->SetValue(CreateExpectedValue());
			};

		ExpectedJsonValue = CreateExpectedJsonValue(InCreateExpectedValue());
	}

	// NOTE: This intentionally doesn't use END_DEFINE_SPEC() as we want to avoid instantiating
	// this class.
};

// Create a test case for a async result with a value type that can be initialized on construction
// from a constant.
// 
// @param AsyncResultValueShortName is the short name of the test case, this is typically the type
//   under test.
// @param AsyncResultType is the type of the UToolCallAsyncResult subclass being tested.
// @param ValueType is type the value of the value that is set to complete the result.
// @param ExpectedValueFactory is a callable that should create the expected value.
// @param ExpectedJsonValueFactory is a callable that takes a reference to the expected value
//   and return TSharedPtr<FJsonValue> to verify the result's value is correctly converted to the
//   expected JSON value.
#define UE_TOOLSET_REGISTRY_TOOL_CALL_ASYNC_RESULT_WITH_VALUE_SPEC(                                \
		AsyncResultValueShortName, AsyncResultType, ValueType, ExpectedValueFactory,               \
		ExpectedJsonValueFactory)                                                                  \
	class FToolCallAsyncResult##AsyncResultValueShortName##Spec :                                  \
		public FToolCallAsyncResultBaseSpec                                                        \
	{                                                                                              \
	public:                                                                                        \
		using BaseSpec::BaseSpec;                                                                  \
	                                                                                               \
		virtual void Define() override                                                             \
		{                                                                                          \
			Setup<AsyncResultType, ValueType>(JsonValueFactory, ValueFactory);                     \
			BaseSpec::Define();                                                                    \
		}                                                                                          \
	                                                                                               \
		virtual FString GetBeautifiedTestName() const override                                     \
		{                                                                                          \
			return TEXT("AI.ToolsetRegistry.ToolCallAsyncResult."                                  \
			            #AsyncResultValueShortName);                                               \
		}                                                                                          \
		                                                                                           \
		TFunction<ValueType()> ValueFactory = ExpectedValueFactory;                                \
		TFunction<TSharedPtr<FJsonValue>(const ValueType&)> JsonValueFactory =                     \
			ExpectedJsonValueFactory;                                                              \
	};                                                                                             \
	                                                                                               \
	namespace                                                                                      \
	{                                                                                              \
		FToolCallAsyncResult##AsyncResultValueShortName##Spec                                      \
		    FToolCallAsyncResult##AsyncResultValueShortName##SpecInstance(                         \
		        TEXT(UE_INLINE_STRINGIFY(                                                          \
		            FToolCallAsyncResultWith##AsyncResultValueShortName##Spec)));                  \
	}

#endif  // WITH_DEV_AUTOMATION_TESTS

// Used to test matching UFunction that returns UToolCallAsyncResult.
UCLASS(BlueprintType, Hidden)
class UToolCallAsyncResultReturnPropertyTest : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	// Async function that returns a result that resolves to a greeting.
	UFUNCTION()
	static UToolCallAsyncResultString* SlowGreeting(const FString& Subject, float DelayInSeconds)
	{
		TObjectPtr<UToolCallAsyncResultString> Result =
			NewObject<UToolCallAsyncResultString>();
		RunAfterDelay(
			[Result = TStrongObjectPtr(Result.Get()), Subject = FString(Subject)]() -> void
			{
				Result->SetValue(FString::Printf(TEXT("Hello %s"), *Subject));
			},
			DelayInSeconds);
		return Result.Get();
	}

	// Async function that return a result that resolves to an error.
	UFUNCTION()
	static UToolCallAsyncResultVoid* SlowError(const FString& Error, float DelayInSeconds)
	{
		TObjectPtr<UToolCallAsyncResultVoid> Result =
			NewObject<UToolCallAsyncResultVoid>();
		RunAfterDelay(
			[Result = TStrongObjectPtr(Result.Get()), Error = FString(Error)]() -> void
			{
				Result->SetError(Error);
			},
			DelayInSeconds);
		return Result.Get();
	}

	// Broken async function that doesn't return a result instance.
	UFUNCTION()
	static UToolCallAsyncResultVoid* Broken()
	{
		return nullptr;
	}

	static UFunction* FindTestFunction()
	{
		return StaticClass()->FindFunctionByName(TEXT("SlowGreeting"));
	}

private:
	// Run a callable after a delay.
	static void RunAfterDelay(TFunction<void()>&& Callable, float DelayInSeconds)
	{
		Async(
			EAsyncExecution::Thread,
			[DelayInSeconds, Callable = MoveTemp(Callable)]()
			{
				// ConditionalSleep polls the condition once before sleeping the thread.
				bool bPolledOnce = false;
				FGenericPlatformProcess::ConditionalSleep(
					[&bPolledOnce]() -> bool
					{
						bool bStopPolling = bPolledOnce;
						bPolledOnce = true;
						return bStopPolling;
					},
					DelayInSeconds);
				Callable();
			});
	}
};
