// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/JsonObject.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "ToolsetRegistry/ToolCallAsyncResultVoid.h"
#include "ToolsetRegistry/JsonConversion.h"
#include "Tests/ToolCallAsyncResultTest.h"

#if WITH_DEV_AUTOMATION_TESTS

class FToolCallAsyncResultVoidSpec :
	public FToolCallAsyncResultBaseSpec
{
public:
	using BaseSpec::BaseSpec;

	virtual void Define() override
	{
		SetupCreateResult<UToolCallAsyncResultVoid>();
		SetResultValue = [this]()
			{
				return CastChecked<UToolCallAsyncResultVoid>(Result)->SetCompleted();
			};
		ExpectedJsonValue = MakeShared<FJsonValueNull>();
		BaseSpec::Define();

		It(TEXT("Should have a null JSON schema for the value"), [this]()
		{
			TestEqual(
				TEXT("JSON schema"),
				UE::ToolsetRegistry::Internal::JsonToString(
					UToolCallAsyncResultVoid::GetValueJsonSchema()),
				TEXT(R"json({"description":"Always null","type":"null"})json"));
		});
	}

	virtual FString GetBeautifiedTestName() const override
	{
		return TEXT("AI.ToolsetRegistry.ToolCallAsyncResult.Void");
	}
};

namespace
{
	FToolCallAsyncResultVoidSpec
		FToolCallAsyncResultVoidSpecInstance(
			TEXT("FToolCallAsyncResultVoidSpec"));
}

#endif  // WITH_DEV_AUTOMATION_TESTS