// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolCallExceptionHandlerTest.h"

#include "Internationalization/Text.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationTest.h"

#include "Tests/ToolCallTestHelpers.h"
#include "Tests/ToolsetRegistryTestFlags.h"
#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/ObjectFunctionToolCall.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"
#include "ToolsetRegistry/ToolCallExceptionHandlerInternal.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FToolCallExceptionHandlerSpec,
	"AI.ToolsetRegistry.ToolCallExceptionHandler",
	ToolsetRegistryTest::Flags)

	// Throw a blueprint exception without an error message.
	static void ThrowScriptException(EBlueprintExceptionType::Type ExceptionType)
	{
		FFrame* TopFrame = FFrame::GetThreadLocalTopStackFrame();
		check(TopFrame);
		FBlueprintCoreDelegates::ThrowScriptException(
			TopFrame->Object, *TopFrame,
			FBlueprintExceptionInfo(ExceptionType, FText()));
	}
END_DEFINE_SPEC(FToolCallExceptionHandlerSpec)

void FToolCallExceptionHandlerSpec::Define()
{
	using namespace UE::ToolsetRegistry;

	Describe(TEXT("GetException"), [this]()
	{
		It(TEXT("Captures an error raised by a function"), [this]()
		{
			FString Error = TEXT("an error");
			FToolCallExceptionHandler Handler;
			Handler.CaptureErrorsIn(
				[&Error]()
				{
					UKismetSystemLibrary::RaiseScriptError(Error);
				});
			TestEqual(TEXT("Error matches"), Handler.GetException(), Error);
		});

		It(TEXT("Accumulates multiple errors and separates them with newlines"), [this]()
		{
			FString Error1 = TEXT("an error");
			FString Error2 = TEXT("another error");
			FToolCallExceptionHandler Handler;
			Handler.CaptureErrorsIn(
				[&Error1, &Error2]()
				{
					UKismetSystemLibrary::RaiseScriptError(Error1);
					UKismetSystemLibrary::RaiseScriptError(Error2);
				});
			TestEqual(TEXT("Error matches"), Handler.GetException(), Error1 + TEXT("\n") + Error2);
		});

		It(TEXT("Reports errors when no error message is provided"), [this]()
		{
			for (const auto& ExceptionToErrorString :
				UE::ToolsetRegistry::Internal::HandledBlueprintExceptionTypeToErrorStrings)
			{
				FToolCallExceptionHandler Handler;
				Handler.CaptureErrorsIn(
					[ExceptionType = ExceptionToErrorString.Key]()
					{
						ThrowScriptException(ExceptionType);
					});
				FString ExceptionMessage = Handler.GetException();
				TestNotEqual(
					*FString::Printf(TEXT("Error reported for '%s'"), *ExceptionToErrorString.Value),
					*ExceptionMessage, TEXT(""));
				TestTrue(
					*FString::Printf(TEXT("Error contains '%s'"), *ExceptionToErrorString.Value),
					ExceptionMessage.Contains(*ExceptionToErrorString.Value));
				
			}
		});

		It(TEXT("Doesn't report errors for ignored exceptions"), [this]()
		{
			for (const auto& ExceptionType :
				UE::ToolsetRegistry::Internal::IgnoredBlueprintExceptionTypes)
			{
				FToolCallExceptionHandler Handler;
				Handler.CaptureErrorsIn(
					[ExceptionType]()
					{
						ThrowScriptException(ExceptionType);
					});
				TestEqual(TEXT("No error"), *Handler.GetException(), TEXT(""));
			}
		});

		It(TEXT("Doesn't catch errors outside of the object lifetime"), [this]()
		{
			auto HandlerCatchesNoErrors = [this]()
				{
					FToolCallExceptionHandler Handler;
					TestEqual(TEXT("No error"), *Handler.GetException(), TEXT(""));
				};
			HandlerCatchesNoErrors();
			UE::ToolsetRegistry::Internal::CallWithinBlueprintScript(
				[]()
				{
					UKismetSystemLibrary::RaiseScriptError();
				});
			HandlerCatchesNoErrors();
		});
	});

	Describe(TEXT("ToolCalls"), [this]()
	{
		It(TEXT("Accumulates multiple parameter errors with newlines"), [this]()
		{
			TObjectPtr<UToolCallExceptionHandlerTestObject> TestObject =
				NewObject<UToolCallExceptionHandlerTestObject>();

			// Pass two invalid references to trigger multiple RaiseScriptError calls
			FString InvalidJson = FString::Printf(
				TEXT(R"({"firstObject": {"refPath": "%s"}, "secondObject": {"refPath": "%s"}})"),
				*UObject::StaticClass()->GetPathName(),  // Wrong type for FirstObject
				*UClass::StaticClass()->GetPathName());  // Wrong type for SecondObject

			FJsonValueOrError Result = TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestMultipleRefs"), InvalidJson);

			if (!TestTrue(TEXT("Should have error"), Result.HasError())) return;

			FString ErrorMessage = Result.GetError();

			// Verify both errors are present
			TestTrue(TEXT("Contains first param type validation error"),
					 ErrorMessage.Contains(
						 TEXT("is not valid ToolCallExceptionHandlerTestObject for "
							 "property 'FirstObject'")));
			TestTrue(TEXT("Contains second param type validation error"),
					 ErrorMessage.Contains(
						 TEXT("is not valid ToolCallExceptionHandlerTestObject for "
							 "property 'SecondObject'")));

			// Verify errors are accumulated with newlines
			TestTrue(TEXT("Contains newline separator"),
					 ErrorMessage.Contains(TEXT("\n")));

			// Verify parameter error prefix
			TestTrue(TEXT("Contains parameter error prefix"),
					 ErrorMessage.Contains(TEXT("Parameter error")));
		});

		It(TEXT("Captures single parameter error correctly"), [this]()
		{
			TObjectPtr<UToolCallExceptionHandlerTestObject> TestObject =
				NewObject<UToolCallExceptionHandlerTestObject>();

			// Pass one invalid reference
			FString InvalidJson = FString::Printf(
				TEXT(R"({"testObject": {"refPath": "%s"}})"),
				*UObject::StaticClass()->GetPathName());  // Wrong type

			FJsonValueOrError Result = TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestSpecObjectParam"), InvalidJson);

			if (!TestTrue(TEXT("Should have error"), Result.HasError())) return;

			FString ErrorMessage = Result.GetError();

			// Verify error message format
			TestTrue(TEXT("Contains type validation error"),
					 ErrorMessage.Contains(
						 TEXT("is not valid ToolCallExceptionHandlerTestObject")));
			TestTrue(TEXT("Error has parameter prefix"),
					 ErrorMessage.Contains(TEXT("Parameter error")));
		});

		It(TEXT("Returns no error when all parameters are valid"), [this]()
		{
			TObjectPtr<UToolCallExceptionHandlerTestObject> TestObject =
				NewObject<UToolCallExceptionHandlerTestObject>();
			TObjectPtr<UToolCallExceptionHandlerTestObject> ValidObject1 =
				NewObject<UToolCallExceptionHandlerTestObject>();
			TObjectPtr<UToolCallExceptionHandlerTestObject> ValidObject2 =
				NewObject<UToolCallExceptionHandlerTestObject>();

			// Pass valid references
			FString ValidJson = FString::Printf(
				TEXT(R"({"firstObject": {"refPath": "%s"}, "secondObject": {"refPath": "%s"}})"),
				*ValidObject1->GetPathName(),
				*ValidObject2->GetPathName());

			FJsonValueOrError Result = TestHelpers::ExecuteToolCallWithJson(
				TestObject, TEXT("TestMultipleRefs"), ValidJson);

			// Should succeed with no error
			TestFalse(TEXT("Should not have error"), Result.HasError());
		});

		It(TEXT("Returns a fallback message when an empty error message is raised"), [this]()
		{
			TObjectPtr<UToolCallExceptionHandlerTestObject> TestObject =
				NewObject<UToolCallExceptionHandlerTestObject>();

			UFunction* Function = TestObject->GetClass()->FindFunctionByName(TEXT("TestEmptyError"));
			if (!TestNotNull(TEXT("Function exists"), Function)) return;

			auto ToolCall = MakeShared<FObjectFunctionToolCall>(TestObject, Function);
			TSharedPtr<FToolCallExceptionHandler> ExceptionHandler =
				MakeShared<FToolCallExceptionHandler>();

			ToolCall->Execute(
				FObjectFunctionToolCall::FFunctionInputParamsJson(TInPlaceType<FString>(), FString()),
				ExceptionHandler);

			FString ErrorMessage = ExceptionHandler->GetException();

			if (!TestFalse(TEXT("Should have error"), ErrorMessage.IsEmpty())) return;

			// Verify error message format
			TestTrue(TEXT("Error has fallback format"),
				ErrorMessage.Contains(TEXT("UserRaisedError in ")));
		});
	});
}

#endif

PRAGMA_ENABLE_DEPRECATION_WARNINGS
