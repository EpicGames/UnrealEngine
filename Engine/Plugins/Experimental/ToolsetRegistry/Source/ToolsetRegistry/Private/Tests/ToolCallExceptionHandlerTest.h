// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/KismetSystemLibrary.h"

#include "ToolCallExceptionHandlerTest.generated.h"


UCLASS(BlueprintType)
class UToolCallExceptionHandlerTestObject : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category="ToolCallExceptionHandlerTest")
	void TestSpecObjectParam(UToolCallExceptionHandlerTestObject* TestObject) {}

	UFUNCTION(BlueprintCallable, Category="ToolCallExceptionHandlerTest")
	void TestMultipleRefs(
		UToolCallExceptionHandlerTestObject* FirstObject,
		UToolCallExceptionHandlerTestObject* SecondObject) {}

	UFUNCTION(BlueprintCallable, Category="ToolCallExceptionHandlerTest")
	void TestEmptyError()
	{
		// Raise an explicitly empty error
		UKismetSystemLibrary::RaiseScriptError(FString());
	}
};
