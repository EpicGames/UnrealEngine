// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/ValueOrError.h"
#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/ObjectFunctionToolCall.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"

namespace UE::ToolsetRegistry::TestHelpers
{
	// Helper to execute a tool call for parameter validation.  Instantiates a tool call script exception
	// handler to capture errors raised during parameter validation before invoking the function.
	inline FJsonValueOrError ExecuteToolCallWithJson(
		UObject* TestObject,
		const FString& FunctionName,
		const FString& InputJson)
	{
		check(TestObject);
		UFunction* Function = TestObject->GetClass()->FindFunctionByName(*FunctionName);
		if (!Function)
		{
			return MakeError(FString(TEXT("Function not found")));
		}

		auto ToolCall = MakeShared<FObjectFunctionToolCall>(TestObject, Function);
		TSharedPtr<FToolCallExceptionHandler> ExceptionHandler =
			MakeShared<FToolCallExceptionHandler>();

		TFuture<FJsonValueOrError> ResultFuture = ToolCall->Execute(
			FObjectFunctionToolCall::FFunctionInputParamsJson(
				TInPlaceType<FString>(), InputJson),
			ExceptionHandler);

		return ResultFuture.Get();
	}
}
