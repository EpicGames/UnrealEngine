// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetJsonConverterTest.h"

#include "CoreMinimal.h"
#include "JsonObjectConverter.h"
#include "Misc/AutomationTest.h"

#include "ToolsetRegistry/ToolsetJsonConverter.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	using namespace UE::ToolsetRegistry;

	// A converter for FDelegatingConverterTestStruct that demonstrates the protected helpers:
	// it omits HiddenField entirely and processes the remaining properties through the full
	// ToolsetJson pipeline, so that registered converters (e.g. FToolsetColorConverter) are
	// applied to sub-properties.
	class FDelegatingConverter : public FToolsetJsonConverter
	{
	public:
		virtual FString GetName() const override { return TEXT("DelegatingConverter"); }

		virtual bool CanConvertProperty(TNotNull<const FProperty*> Property) override
		{
			const FStructProperty* StructProp = CastField<FStructProperty>(Property);
			return StructProp &&
				StructProp->Struct == FDelegatingConverterTestStruct::StaticStruct();
		}

		virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(
			TNotNull<const FProperty*> Property) override
		{
			TSharedPtr<FJsonObject> Schema = MakeShared<FJsonObject>();
			Schema->SetStringField(TEXT("type"), TEXT("object"));
			TSharedPtr<FJsonObject> Properties = MakeShared<FJsonObject>();

			for (TFieldIterator<FProperty> It(FDelegatingConverterTestStruct::StaticStruct());
				It; ++It)
			{
				if ((*It)->GetFName() == GET_MEMBER_NAME_CHECKED(
					FDelegatingConverterTestStruct, HiddenField))
				{
					continue;
				}
				FString JsonName = FJsonObjectConverter::StandardizeCase((*It)->GetName());
				Properties->SetObjectField(JsonName, ToolsetPropertyToJsonSchema(*It));
			}

			Schema->SetObjectField(TEXT("properties"), Properties);
			return Schema;
		}

		// This is not exercised in this test.
		virtual TSharedPtr<FJsonValue> PropertyToDefault(
			TNotNull<const FProperty*>, const FString&) override { return nullptr; }

		virtual TSharedPtr<FJsonValue> PropertyToJsonData(
			TNotNull<FProperty*> Property, const void* Value) override
		{
			const FDelegatingConverterTestStruct* Struct =
				static_cast<const FDelegatingConverterTestStruct*>(Value);
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

			for (TFieldIterator<FProperty> It(FDelegatingConverterTestStruct::StaticStruct());
				It; ++It)
			{
				if ((*It)->GetFName() == GET_MEMBER_NAME_CHECKED(
					FDelegatingConverterTestStruct, HiddenField))
				{
					continue;
				}
				FString JsonName = FJsonObjectConverter::StandardizeCase((*It)->GetName());
				const uint8* SubValue = (*It)->ContainerPtrToValuePtr<uint8>(Struct);
				TSharedPtr<FJsonValue> FieldJson = ToolsetPropertyToJsonData(*It, SubValue);
				if (FieldJson.IsValid())
				{
					JsonObject->SetField(JsonName, FieldJson);
				}
			}

			return MakeShared<FJsonValueObject>(JsonObject);
		}

		virtual bool JsonDataToProperty(
			const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
			void* OutValue, UObject* Outer) override
		{
			TSharedPtr<FJsonObject> JsonObject = JsonValue->AsObject();
			if (!JsonObject.IsValid()) return false;

			FDelegatingConverterTestStruct* Struct =
				static_cast<FDelegatingConverterTestStruct*>(OutValue);

			for (TFieldIterator<FProperty> It(FDelegatingConverterTestStruct::StaticStruct());
				It; ++It)
			{
				if ((*It)->GetFName() == GET_MEMBER_NAME_CHECKED(
					FDelegatingConverterTestStruct, HiddenField))
				{
					continue;
				}
				FString JsonName = FJsonObjectConverter::StandardizeCase((*It)->GetName());
				TSharedPtr<FJsonValue> FieldJson = JsonObject->TryGetField(JsonName);
				if (FieldJson.IsValid())
				{
					uint8* SubValue = (*It)->ContainerPtrToValuePtr<uint8>(Struct);
					ToolsetJsonDataToProperty(FieldJson, *It, SubValue);
				}
			}

			return true;
		}
	};
}

BEGIN_DEFINE_SPEC(FToolsetJsonConverterSpec, "AI.ToolsetRegistry.ToolsetJsonConverterSpec",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolsetJsonConverterSpec)

void FToolsetJsonConverterSpec::Define()
{
	It(TEXT("Schema excludes hidden field and applies registered converter to remaining fields"),
		[this]()
	{
		FDelegatingConverter Converter;
		FProperty* StructProp = FDelegatingConverterTestContainer::StaticStruct()
			->FindPropertyByName(TEXT("TestStruct"));
		if (!TestTrue(TEXT("Has property"), StructProp != nullptr)) return;

		TSharedPtr<FJsonObject> Schema = Converter.PropertyToJsonSchema(StructProp);
		if (!TestTrue(TEXT("Schema"), Schema.IsValid())) return;

		const TSharedPtr<FJsonObject>* Properties = nullptr;
		if (!TestTrue(TEXT("Properties"), Schema->TryGetObjectField(TEXT("properties"), Properties)))
			return;

		TestFalse(TEXT("HiddenField absent"), Properties->Get()->HasField(TEXT("hiddenField")));

		// The color field should carry the FToolsetColorConverter schema (title=LinearColor
		// with rgba components), not the generic FColor struct schema.
		const TSharedPtr<FJsonObject>* ColorSchema = nullptr;
		if (!TestTrue(TEXT("color present"), Properties->Get()->TryGetObjectField(
			TEXT("color"), ColorSchema))) return;

		FString Title;
		TestTrue(TEXT("Color title"), ColorSchema->Get()->TryGetStringField(TEXT("title"), Title));
		TestEqual(TEXT("Color title"), Title, FString(TEXT("LinearColor")));

		const TSharedPtr<FJsonObject>* ColorProperties = nullptr;
		if (!TestTrue(TEXT("Color properties"), ColorSchema->Get()->TryGetObjectField(
			TEXT("properties"), ColorProperties))) return;
		for (const FString& Component : TArray<FString>{ TEXT("r"), TEXT("g"), TEXT("b"), TEXT("a") })
		{
			const TSharedPtr<FJsonObject>* ComponentSchema = nullptr;
			TestTrue(*Component, ColorProperties->Get()->TryGetObjectField(Component, ComponentSchema));
		}
	});

	It(TEXT("Data excludes hidden field, applies registered converter, and round-trips correctly"),
		[this]()
	{
		FDelegatingConverter Converter;
		FProperty* StructProp = FDelegatingConverterTestContainer::StaticStruct()
			->FindPropertyByName(TEXT("TestStruct"));
		if (!TestTrue(TEXT("Has property"), StructProp != nullptr)) return;

		FDelegatingConverterTestStruct TestIn;
		TestIn.HiddenField = TEXT("secret");
		TestIn.Color = FColor(10, 20, 30, 40);

		TSharedPtr<FJsonValue> Json = Converter.PropertyToJsonData(StructProp, &TestIn);
		if (!TestTrue(TEXT("Json"), Json.IsValid())) return;

		TSharedPtr<FJsonObject> JsonObject = Json->AsObject();
		if (!TestTrue(TEXT("Json object"), JsonObject.IsValid())) return;

		TestFalse(TEXT("HiddenField absent from data"), JsonObject->HasField(TEXT("hiddenField")));

		// The color should be serialized as RGBA by FToolsetColorConverter, not as a
		// generic FColor.
		TestTrue(TEXT("color present"), JsonObject->HasField(TEXT("color")));

		FDelegatingConverterTestStruct TestOut;
		TestOut.HiddenField = TEXT("untouched");
		TestTrue(TEXT("Converted"), Converter.JsonDataToProperty(Json, StructProp, &TestOut, nullptr));

		TestEqual(TEXT("Color"), TestIn.Color, TestOut.Color);
		TestEqual(TEXT("HiddenField unchanged"), TestOut.HiddenField, FString(TEXT("untouched")));
	});
}

#endif
