// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolsetJsonConverterTest.generated.h"

// Test struct whose converter hides HiddenField and delegates Color to the Toolset pipeline.
USTRUCT()
struct FDelegatingConverterTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FString HiddenField;

	UPROPERTY()
	FColor Color = FColor::White;
};

// Container used to obtain an FStructProperty of type FDelegatingConverterTestStruct.
USTRUCT()
struct FDelegatingConverterTestContainer
{
	GENERATED_BODY()

	UPROPERTY()
	FDelegatingConverterTestStruct TestStruct;
};

