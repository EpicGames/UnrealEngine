// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetJsonTest.h"

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "Misc/AutomationTest.h"

#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/JsonSchema.h"
#include "ToolsetRegistry/ToolsetJson.h"
#include "ToolsetRegistry/ToolCallAsyncResultString.h"
#include "Tests/ToolCallAsyncResultTest.h"

void UToolsetContainerTestObject::PostEditChangeChainProperty(FPropertyChangedChainEvent& Event)
{
	Super::PostEditChangeChainProperty(Event);

	FRecordedChangeEvent& Rec = RecordedNotifications.Emplace_GetRef();
	Rec.ChangeType = Event.ChangeType;

	if (const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ActiveNode =
		Event.PropertyChain.GetActiveNode())
	{
		if (FProperty* Prop = ActiveNode->GetValue())
		{
			Rec.ActivePropertyName = Prop->GetName();
		}
	}
	if (const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* MemberNode =
		Event.PropertyChain.GetActiveMemberNode())
	{
		if (FProperty* Prop = MemberNode->GetValue())
		{
			Rec.MemberPropertyName = Prop->GetName();
		}
	}

	// Collect element index for each property node in the chain.
	for (const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* Node =
		Event.PropertyChain.GetHead(); Node != nullptr; Node = Node->GetNextNode())
	{
		if (FProperty* Prop = Node->GetValue())
		{
			const int32 Idx = Event.GetArrayIndex(Prop->GetName());
			if (Idx != INDEX_NONE)
			{
				Rec.ElementIndices.Add(Prop->GetName(), Idx);
			}
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	TSharedPtr<FJsonObject> TestHasObjectField(
		FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema,
		const FString& RootObjectName, const TArrayView<FString>& FieldPath)
	{
		const TSharedPtr<FJsonObject>* MaybeObjectField = nullptr;
		TSharedPtr<FJsonObject> OutputObject;
		verify(FieldPath.Num() > 0);
		FString FieldName = FieldPath[0];
		FString ParentName =
			FieldPath.Num() > 1
			? FString::Printf(TEXT("%s.%s"), *RootObjectName, *FieldName)
			: RootObjectName;
		if (Schema.IsValid() &&
			Test.TestTrue(
				FString::Printf(TEXT("%s has field %s"), *ParentName, *FieldName),
				Schema->TryGetObjectField(*FieldName, MaybeObjectField) &&
				MaybeObjectField && MaybeObjectField->IsValid()) &&
			MaybeObjectField)
		{
			OutputObject =
				FieldPath.Num() > 1
				? TestHasObjectField(
					Test, *MaybeObjectField, ParentName, FieldPath.Right(FieldPath.Num() - 1))
				: *MaybeObjectField;
		}
		return OutputObject;
	}
}

BEGIN_DEFINE_SPEC(FToolsetJsonSpec, "AI.ToolsetRegistry.ToolsetJsonSpec",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolsetJsonSpec)

void FToolsetJsonSpec::Define()
{
	using namespace UE::ToolsetRegistry::Internal;
	using namespace UE::ToolsetRegistry::Internal::ToolsetJson;

	It("Filters user visible properties schema", [this]()
	{
		TSharedPtr<FJsonObject> ClassSchema = StructToJsonSchema(
			UToolsetJsonTestObject::StaticClass(), true);
		if (!TestTrue("TestData", ClassSchema.IsValid())) return;
		const TSharedPtr<FJsonObject>* Properties = nullptr;
		if (!(TestTrue("Properties", ClassSchema->TryGetObjectField(TEXT("properties"), Properties)) &&
			Properties))
		{
			return;
		}
		TestTrue("Visible", Properties->Get()->HasField(TEXT("visibleProperty")));
		TestFalse("Hidden", Properties->Get()->HasField(TEXT("hiddenProperty")));
	});

	It("Filters user visible properties data", [this]()
	{
		TSharedPtr<FJsonObject> TestData = StructToJsonData(
			UToolsetJsonTestObject::StaticClass(), NewObject<UToolsetJsonTestObject>(), true);
		if (!TestTrue("TestData", TestData.IsValid())) return;
		TestTrue("Visible", TestData->HasField(TEXT("visibleProperty")));
		TestFalse("Hidden", TestData->HasField(TEXT("hiddenProperty")));
	});

	It(TEXT("Converts ToolCallAsyncResult return value to JSON schema"), [this]()
	{
		UFunction* Function = UToolCallAsyncResultReturnPropertyTest::FindTestFunction();
		if (!TestTrue(TEXT("Has test function"), Function != nullptr)) return;

		const TSharedPtr<FJsonObject> Schema = StructToJsonSchema(Function);
		const TSharedPtr<FJsonObject> ReturnValueSchema =
			TestHasObjectField(
				*this, Schema, TEXT("Function schema"),
				TArray<FString>{ TEXT("outputSchema"), TEXT("properties"), TEXT("returnValue") });
		if (!ReturnValueSchema) return;

		TestEqual(
			TEXT("Matches expected schema"),
			JsonToString(ReturnValueSchema.ToSharedRef()),
			JsonToString(PropertyToJsonSchema(Function->GetReturnProperty()).ToSharedRef()));
	});

	It(TEXT("Converts ToolCallAsyncResult to JSON"), [this]()
	{
		TObjectPtr<UToolCallAsyncResultString> ToolCallAsyncResult =
			NewObject<UToolCallAsyncResultString>();
		ToolCallAsyncResult->SetValue(TEXT("hello world"));

		FToolsetJsonTest TestIn;
		TestIn.TestToolCallAsyncResult = ToolCallAsyncResult;
		UStruct* Struct = FToolsetJsonTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName(TEXT("TestToolCallAsyncResult"));
		if (!TestTrue(TEXT("Has property"), Property != nullptr)) return;

		TSharedPtr<FJsonValue> JsonValue = PropertyToJsonData(
			Property, &TestIn.TestToolCallAsyncResult);
		if (!TestTrue(TEXT("Converted return value to JSON"), JsonValue.IsValid())) return;

		TestEqual(
			TEXT("Matches value JSON"),
			JsonToString(JsonValue.ToSharedRef()),
			ToolCallAsyncResult->GetValueAsJsonString());
	});
}

#endif
