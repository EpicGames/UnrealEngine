// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/UnrealType.h"

#include "ToolsetRegistry/ToolCallAsyncResult.h"

#include "ToolsetJsonTest.generated.h"


UCLASS(BlueprintType)
class UToolsetJsonTestObject : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	bool VisibleProperty;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	FVector StructProperty;

	// No CPF_Edit/BlueprintVisible/BlueprintAssignable → AccessProtected for both get and set.
	UPROPERTY()
	bool HiddenProperty;

	// CPF_Edit | CPF_EditConst → accessible for get, ReadOnly for set.
	UPROPERTY(VisibleAnywhere, Category="ToolsetTest")
	bool ReadOnlyProperty;

	// CPF_Edit | CPF_DisableEditOnInstance → accessible for get, CannotEditInstance on non-templates.
	UPROPERTY(EditDefaultsOnly, Category="ToolsetTest")
	bool DefaultsOnlyProperty;

	// CPF_Edit | CPF_DisableEditOnTemplate → accessible for get, CannotEditTemplate on templates (CDOs).
	UPROPERTY(EditInstanceOnly, Category="ToolsetTest")
	bool InstanceOnlyProperty;
};

USTRUCT(BlueprintType)
struct FToolsetJsonTest
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TObjectPtr<UToolCallAsyncResult> TestToolCallAsyncResult;
};

// ---------------------------------------------------------------------------
// Test fixtures for SetObjectProperties container notification tests
// ---------------------------------------------------------------------------

/** Inner struct used for nested container notification tests. */
USTRUCT(BlueprintType)
struct FToolsetLibraryInnerStruct
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	TArray<int32> InnerItems;
};

/** Struct used for sibling-property notification tests (plain field + array in same struct). */
USTRUCT(BlueprintType)
struct FToolsetLibraryTestStruct
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	FString Name;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	TArray<int32> Items;
};

/**
 * Two-level nested struct used for verifying that the unmatched-key path builder reports
 * the full chain (Outer.Inner.unmatchedKey) and that the per-chain-depth path is collision-free
 * when the same field name appears at multiple struct levels.
 */
USTRUCT(BlueprintType)
struct FToolsetLibraryNestedStruct
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	FString Name;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	FToolsetLibraryTestStruct Inner;
};

/** One captured PostEditChangeChainProperty event. */
struct FRecordedChangeEvent
{
	EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified;
	/** GetName() of the innermost active property in the chain. */
	FString ActivePropertyName;
	/** GetName() of the outermost (UObject member) property in the chain. */
	FString MemberPropertyName;
	/** Element index per container property name, from the event's ArrayIndicesPerObject map. */
	TMap<FString, int32> ElementIndices;
};

/** Test UObject that records every PostEditChangeChainProperty call for inspection. */
UCLASS()
class UToolsetContainerTestObject : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	TArray<int32> IntArray;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	TSet<int32> IntSet;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	TMap<FString, int32> IntMap;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	TMap<FString, FToolsetLibraryInnerStruct> StructMap;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	FToolsetLibraryTestStruct StructProp;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	TArray<FToolsetLibraryInnerStruct> OuterArray;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	FToolsetLibraryNestedStruct NestedStruct;

	/**
	 * Bare object reference used to exercise the ReferenceConverter rejection path through
	 * SetObjectProperties (i.e. to verify the GConverterRejectedInput RAII guard via the
	 * end-to-end API). The property must be BlueprintReadWrite so PropertyAccessUtil grants
	 * write access; the converter's class-resolution branch then runs and can reject.
	 */
	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	TObjectPtr<UObject> RefObj;

	// FTransform has a registered FToolsetJsonConverter (FToolsetTransformConverter), so
	// ToolsetJson serializes it via FToolsetTransform's {location,rotation,scale} schema
	// rather than the native FProperty layout. Used to verify that struct properties /
	// container elements with a custom importer skip field-by-field recursion.
	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	FTransform TransformProp;

	UPROPERTY(BlueprintReadWrite, Category="ToolsetTest")
	TArray<FTransform> TransformArray;

	TArray<FRecordedChangeEvent> RecordedNotifications;

	void ResetNotifications() { RecordedNotifications.Reset(); }

	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& Event) override;
};
