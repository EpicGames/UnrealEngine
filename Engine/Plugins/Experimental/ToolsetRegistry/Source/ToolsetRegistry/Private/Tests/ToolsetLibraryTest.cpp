// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Dom/JsonValue.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/TransBuffer.h"
#include "JsonObjectConverter.h"
#include "Misc/AutomationTest.h"

#include "Animation/AnimNode_Root.h"
#include "Tests/FunctionLibraryToolsetTest.h"
#include "ToolsetJsonTest.h"
#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/ToolsetLibrary.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"

#if WITH_DEV_AUTOMATION_TESTS


BEGIN_DEFINE_SPEC(FToolsetLibrarySpec, "AI.ToolsetRegistry.ToolsetLibrarySpec",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolsetLibrarySpec)

void FToolsetLibrarySpec::Define()
{
	using namespace UE::ToolsetRegistry;
	using namespace UE::ToolsetRegistry::Internal;

	Describe(TEXT("ListStructProperties"), [this]()
	{
		It(TEXT("generates schema from properties."), [this]()
		{
			FString PropertiesJson = UToolsetLibrary::ListStructProperties(
				UToolsetJsonTestObject::StaticClass());
			TSharedPtr<FJsonObject> JsonObject = JsonStringToJsonObject(PropertiesJson);
			TestTrue("Schema", JsonObject.IsValid());
			if (!JsonObject.IsValid())
			{
				return;
			}
			TestTrue("Schema", JsonObject->HasField(TEXT("visibleProperty")));
			TestFalse("Schema", JsonObject->HasField(TEXT("hiddenProperty")));
		});
	});

	Describe(TEXT("GetObjectProperties"), [this]()
	{
		It(TEXT("gets values from properties."), [this]()
		{
			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TestObject->VisibleProperty = true;
			FString PropertiesJson = UToolsetLibrary::GetObjectProperties(
				TestObject, TArray<FName>({ FName("visibleProperty") }));
			TSharedPtr<FJsonObject> JsonObject = JsonStringToJsonObject(PropertiesJson);
			TestTrue("Schema", JsonObject.IsValid());
			if (!JsonObject.IsValid())
			{
				return;
			}
			bool IsVisible = false;
			TestTrue("GetProp", JsonObject->TryGetBoolField(TEXT("visibleProperty"), IsVisible));
			TestTrue("GetProp", IsVisible);
		});

		It(TEXT("raises error with null object."), [this]()
		{
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn(
				[this]()
				{
					FString PropertiesJson = UToolsetLibrary::GetObjectProperties(
						nullptr, TArray<FName>());
					TestTrue(TEXT("No properties"), PropertiesJson.IsEmpty());
				});
			TestNotEqual(TEXT("Raised error"), *ExceptionHandler.GetException(), TEXT(""));
		});

		It(TEXT("omits missing properties from result."), [this]()
		{
			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TestObject->VisibleProperty = true;
			FString PropertiesJson = UToolsetLibrary::GetObjectProperties(
				TestObject, { FName("visibleProperty"), FName("doesNotExist") });
			TSharedPtr<FJsonObject> JsonObject = JsonStringToJsonObject(PropertiesJson);
			TestTrue("Accessible property is present", JsonObject->HasField(TEXT("visibleProperty")));
			TestFalse("Missing property is absent", JsonObject->HasField(TEXT("doesNotExist")));
		});

		It(TEXT("omits access-protected properties from result."), [this]()
		{
			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TestObject->VisibleProperty = true;
			FString PropertiesJson = UToolsetLibrary::GetObjectProperties(
				TestObject, { FName("visibleProperty"), FName("hiddenProperty") });
			TSharedPtr<FJsonObject> JsonObject = JsonStringToJsonObject(PropertiesJson);
			TestTrue("Accessible property is present", JsonObject->HasField(TEXT("visibleProperty")));
			TestFalse("Protected property is absent", JsonObject->HasField(TEXT("hiddenProperty")));
		});
	});

	Describe(TEXT("SetObjectProperties"), [this]()
	{
		// Helper lambda: build a JSON string with one array field.
		auto ArrayJson = [](const FString& FieldName, TArray<int32> Values) -> FString
		{
			TArray<TSharedPtr<FJsonValue>> Items;
			for (int32 V : Values)
			{
				Items.Add(MakeShared<FJsonValueNumber>(V));
			}
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetArrayField(FieldName, Items);
			return UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef());
		};

		// ---------------------------------------------------------------------
		// Basic property setting
		// ---------------------------------------------------------------------

		It(TEXT("sets values on properties."), [this]()
		{
			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TestFalse("SetProp", TestObject->VisibleProperty);
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("visibleProperty"), true);
			FString PropertiesJson = JsonToString(JsonObject.ToSharedRef());
			TestTrue("SetProp", UToolsetLibrary::SetObjectProperties(TestObject, PropertiesJson));
			TestTrue("SetProp", TestObject->VisibleProperty);
		});

		It(TEXT("returns false when setting values on non-existent properties."), [this]()
		{
			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("doesNotExist"), true);
			FString PropertiesJson = JsonToString(JsonObject.ToSharedRef());
			TestFalse("SetProp", UToolsetLibrary::SetObjectProperties(TestObject, PropertiesJson));
		});

		It(TEXT("returns false when setting access-protected properties."), [this]()
		{
			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("hiddenProperty"), true);
			TestFalse("SetProp", UToolsetLibrary::SetObjectProperties(TestObject, JsonToString(JsonObject.ToSharedRef())));
			TestFalse("Property not modified", TestObject->HiddenProperty);
		});

		It(TEXT("returns false when setting read-only properties."), [this]()
		{
			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("readOnlyProperty"), true);
			TestFalse("SetProp", UToolsetLibrary::SetObjectProperties(TestObject, JsonToString(JsonObject.ToSharedRef())));
			TestFalse("Property not modified", TestObject->ReadOnlyProperty);
		});

		It(TEXT("returns false when setting defaults-only properties on instances."), [this]()
		{
			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("defaultsOnlyProperty"), true);
			TestFalse("SetProp", UToolsetLibrary::SetObjectProperties(TestObject, JsonToString(JsonObject.ToSharedRef())));
			TestFalse("Property not modified", TestObject->DefaultsOnlyProperty);
		});

		It(TEXT("returns false when property value cannot be deserialized."), [this]()
		{
			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetStringField(TEXT("structProperty"), TEXT("not-a-vector"));
			// ToolsetJson now wraps the FJsonObjectConverter calls in a LogJson verbosity
			// override, so the generic "Unable to import" log no longer fires when a
			// converter fails. The only user-visible signal for this case is now the
			// top-level "could not be set" script error and the false return value -
			// both are asserted below.
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn(
				[&]()
				{
					TestFalse("SetProp", UToolsetLibrary::SetObjectProperties(
						TestObject, JsonToString(JsonObject.ToSharedRef())));
				});
			TestEqual("Property not modified", TestObject->StructProperty, FVector::ZeroVector);
			TestNotEqual(TEXT("Raised error"), *ExceptionHandler.GetException(), TEXT(""));
			TestTrue(TEXT("Error mentions the failing property"),
				ExceptionHandler.GetException().Contains(TEXT("structProperty")));
		});

		It(TEXT("returns false when setting instance-only properties on templates."), [this]()
		{
			UToolsetJsonTestObject* TestObject =
				UToolsetJsonTestObject::StaticClass()->GetDefaultObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("instanceOnlyProperty"), true);
			TestFalse("SetProp", UToolsetLibrary::SetObjectProperties(TestObject, JsonToString(JsonObject.ToSharedRef())));
			TestFalse("Property not modified", TestObject->InstanceOnlyProperty);
		});

		// ---------------------------------------------------------------------
		// Nested struct key validation: ImportStructFieldsWithNotify previously
		// `continue`d past unknown JSON keys, causing typos like "shake_class" vs
		// "ShakeClass" to silently succeed. The fix accumulates unmatched keys
		// in FPropertyImportContext::OutUnmatchedKeys and surfaces them through
		// the same InaccessibleProperties error path as top-level mismatches.
		// ---------------------------------------------------------------------

		It(TEXT("returns false and reports nested unknown keys when struct contains a misspelled field."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->StructProp.Name = TEXT("original");

			// "name" matches FToolsetLibraryTestStruct::Name; "doesNotExist" is bogus.
			// At least one matching field is required for ImportStructFieldsWithNotify to
			// descend (otherwise the struct falls through to ValueSet as a custom-format opt-in).
			TSharedPtr<FJsonObject> Inner = MakeShared<FJsonObject>();
			Inner->SetStringField(TEXT("name"), TEXT("updated"));
			Inner->SetStringField(TEXT("doesNotExist"), TEXT("ignored"));

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("structProp"), Inner);

			AddExpectedErrorPlain(TEXT("structProp.doesNotExist"),
				EAutomationExpectedErrorFlags::Contains, 1);
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set reports failure"),
					UToolsetLibrary::SetObjectProperties(Obj, JsonToString(Root.ToSharedRef())));
			});
			TestTrue(TEXT("Raised error mentions nested key"),
				ExceptionHandler.GetException().Contains(TEXT("structProp.doesNotExist")));
			// Matching field still applied — partial-update semantics, with the
			// failure reported via the bool return so callers can react.
			TestEqual(TEXT("Matching field applied"), Obj->StructProp.Name, FString(TEXT("updated")));
		});

		It(TEXT("returns true when struct contains only known fields."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();

			TSharedPtr<FJsonObject> Inner = MakeShared<FJsonObject>();
			Inner->SetStringField(TEXT("name"), TEXT("new name"));

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("structProp"), Inner);

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, JsonToString(Root.ToSharedRef())));
			TestEqual(TEXT("Name set"), Obj->StructProp.Name, FString(TEXT("new name")));
		});

		It(TEXT("reports multiple unknown keys within the same nested struct."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();

			// "name" matches; "bogusOne" and "bogusTwo" do not.
			TSharedPtr<FJsonObject> Inner = MakeShared<FJsonObject>();
			Inner->SetStringField(TEXT("name"), TEXT("ok"));
			Inner->SetStringField(TEXT("bogusOne"), TEXT("nope"));
			Inner->SetStringField(TEXT("bogusTwo"), TEXT("nope"));

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("structProp"), Inner);

			AddExpectedErrorPlain(TEXT("StructProp.bogusOne"),
				EAutomationExpectedErrorFlags::Contains, 1);
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set reports failure"),
					UToolsetLibrary::SetObjectProperties(Obj, JsonToString(Root.ToSharedRef())));
			});
			TestTrue(TEXT("Error mentions first unknown key"),
				ExceptionHandler.GetException().Contains(TEXT("StructProp.bogusOne")));
			TestTrue(TEXT("Error mentions second unknown key"),
				ExceptionHandler.GetException().Contains(TEXT("StructProp.bogusTwo")));
			TestEqual(TEXT("Matching field still applied"),
				Obj->StructProp.Name, FString(TEXT("ok")));
		});

		It(TEXT("reports unknown keys with array element index in the dotted path."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->OuterArray.AddDefaulted();
			Obj->OuterArray[0].InnerItems = {1, 2, 3};

			// Same array length so element-recursion path runs (no size change); element [0]
			// is a struct whose JSON contains both a matching key and an unknown key.
			TSharedPtr<FJsonObject> ElemJson = MakeShared<FJsonObject>();
			ElemJson->SetArrayField(TEXT("innerItems"), TArray<TSharedPtr<FJsonValue>>{
				MakeShared<FJsonValueNumber>(1), MakeShared<FJsonValueNumber>(2),
				MakeShared<FJsonValueNumber>(3) });
			ElemJson->SetStringField(TEXT("notARealField"), TEXT("ignored"));

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetArrayField(TEXT("outerArray"),
				TArray<TSharedPtr<FJsonValue>>{ MakeShared<FJsonValueObject>(ElemJson) });

			AddExpectedErrorPlain(TEXT("OuterArray[0].notARealField"),
				EAutomationExpectedErrorFlags::Contains, 1);
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set reports failure"),
					UToolsetLibrary::SetObjectProperties(Obj, JsonToString(Root.ToSharedRef())));
			});
			TestTrue(TEXT("Dotted path includes container index"),
				ExceptionHandler.GetException().Contains(TEXT("OuterArray[0].notARealField")));
		});

		It(TEXT("reports unknown keys inside map value structs."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			FToolsetLibraryInnerStruct Existing;
			Existing.InnerItems = {1, 2, 3};
			Obj->StructMap.Add(TEXT("A"), Existing);

			// Keep key "A" with unchanged InnerItems so we go through the element-recursion
			// path rather than triggering an ambiguous mixed-change rejection.
			TSharedPtr<FJsonObject> ValueJson = MakeShared<FJsonObject>();
			ValueJson->SetArrayField(TEXT("innerItems"), TArray<TSharedPtr<FJsonValue>>{
				MakeShared<FJsonValueNumber>(1), MakeShared<FJsonValueNumber>(2),
				MakeShared<FJsonValueNumber>(3) });
			ValueJson->SetStringField(TEXT("misspelled"), TEXT("ignored"));

			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetObjectField(TEXT("A"), ValueJson);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("structMap"), MapJson);

			// Tightened assertion: verify the full dotted path "StructMap[A].misspelled"
			// — not just that StructMap and misspelled appear somewhere. The map key "A"
			// must surface between brackets (M1 fix); a bare logical iteration index would
			// have been meaningless to the caller.
			AddExpectedErrorPlain(TEXT("StructMap[A].misspelled"),
				EAutomationExpectedErrorFlags::Contains, 1);
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set reports failure"),
					UToolsetLibrary::SetObjectProperties(Obj, JsonToString(Root.ToSharedRef())));
			});
			TestTrue(TEXT("Dotted path includes map key in brackets and the unmatched key"),
				ExceptionHandler.GetException().Contains(TEXT("StructMap[A].misspelled")));
		});

		It(TEXT("reports unknown keys two levels deep with full dotted path."), [this]()
		{
			// Verify the path builder accumulates correctly across two struct descents:
			// path should be "NestedStruct.Inner.bogusKey", not "Inner.bogusKey" or
			// "NestedStruct.bogusKey". Also verifies that field names appearing at multiple
			// levels ("name" exists on both outer and inner) don't collide in the path.
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();

			TSharedPtr<FJsonObject> InnerJson = MakeShared<FJsonObject>();
			InnerJson->SetStringField(TEXT("name"), TEXT("inner name"));
			InnerJson->SetStringField(TEXT("bogusKey"), TEXT("nope"));

			TSharedPtr<FJsonObject> OuterJson = MakeShared<FJsonObject>();
			OuterJson->SetStringField(TEXT("name"), TEXT("outer name"));
			OuterJson->SetObjectField(TEXT("inner"), InnerJson);

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("nestedStruct"), OuterJson);

			AddExpectedErrorPlain(TEXT("NestedStruct.Inner.bogusKey"),
				EAutomationExpectedErrorFlags::Contains, 1);
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set reports failure"),
					UToolsetLibrary::SetObjectProperties(Obj, JsonToString(Root.ToSharedRef())));
			});
			TestTrue(TEXT("Full dotted path"),
				ExceptionHandler.GetException().Contains(TEXT("NestedStruct.Inner.bogusKey")));
			TestEqual(TEXT("Outer name applied"),
				Obj->NestedStruct.Name, FString(TEXT("outer name")));
			TestEqual(TEXT("Inner name applied"),
				Obj->NestedStruct.Inner.Name, FString(TEXT("inner name")));
		});

		It(TEXT("does not leak the GConverterRejectedInput thread-local across calls."), [this]()
		{
			// Direct exercise of the FRejectionScope RAII guard: a call that triggers our
			// ReferenceConverter to reject (sets GConverterRejectedInput=true mid-call, which
			// the outer wrapper reads, then the RAII dtor must restore the saved prior value
			// — otherwise the next call's wrapper would observe a stale true and incorrectly
			// fail valid input). Sequence: reject → accept → reject.
			UToolsetContainerTestObject* ObjA = NewObject<UToolsetContainerTestObject>();
			UToolsetContainerTestObject* ObjB = NewObject<UToolsetContainerTestObject>();

			// Bad: a bare UObject reference path that doesn't resolve to anything. Goes through
			// ReferenceConverter's "Object is not valid" rejection branch, which raises a script
			// error and returns false — translating into GConverterRejectedInput = true inside
			// our CustomImport callback.
			TSharedPtr<FJsonObject> BadRef = MakeShared<FJsonObject>();
			BadRef->SetStringField(TEXT("refPath"), TEXT("/Game/NotAValid.Class"));
			TSharedPtr<FJsonObject> BadRoot = MakeShared<FJsonObject>();
			BadRoot->SetObjectField(TEXT("refObj"), BadRef);

			// Good: a path that resolves to an actual UObject (any loaded UClass works since
			// UClass IS-A UObject and the property has no specific subclass constraint).
			TSharedPtr<FJsonObject> GoodRef = MakeShared<FJsonObject>();
			GoodRef->SetStringField(TEXT("refPath"),
				UToolsetContainerTestObject::StaticClass()->GetPathName());
			TSharedPtr<FJsonObject> GoodRoot = MakeShared<FJsonObject>();
			GoodRoot->SetObjectField(TEXT("refObj"), GoodRef);

			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("First rejecting call returns false"),
					UToolsetLibrary::SetObjectProperties(ObjA, JsonToString(BadRoot.ToSharedRef())));
				TestTrue(TEXT("Subsequent valid call is not poisoned by prior rejection"),
					UToolsetLibrary::SetObjectProperties(ObjB, JsonToString(GoodRoot.ToSharedRef())));
				TestNotNull(TEXT("Valid call set the property"), ObjB->RefObj.Get());
				TestFalse(TEXT("Second rejecting call still rejects"),
					UToolsetLibrary::SetObjectProperties(ObjA, JsonToString(BadRoot.ToSharedRef())));
			});
		});

		It(TEXT("falls through to ValueSet for struct JSON with no matching field properties."),
			[this]()
		{
			// Safety guard in ImportPropertyWithNotify: when a struct's JSON contains zero keys
			// that resolve to FProperty fields, we treat it as a custom-format opt-in and emit a
			// single ValueSet for the whole struct (letting FJsonObjectConverter handle it). This
			// covers IJsonObjectStructConverter-style structs and types like FInstancedStruct
			// without us having to enumerate them. The all-unknown JSON here exercises that path:
			// no FProperty matches, so no per-field events are emitted and no unmatched-key
			// errors are surfaced.
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->StructProp.Name = TEXT("original");
			Obj->StructProp.Items = {1, 2, 3};

			TSharedPtr<FJsonObject> Inner = MakeShared<FJsonObject>();
			Inner->SetStringField(TEXT("nothing"), TEXT("matches"));
			Inner->SetStringField(TEXT("anywhere"), TEXT("either"));

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("structProp"), Inner);

			// JsonObjectConverter will discard this since none of the keys map to a UPROPERTY,
			// but the call should succeed (no rejection, no unmatched-key errors).
			TestTrue(TEXT("Set succeeded (struct fell through to ValueSet)"),
				UToolsetLibrary::SetObjectProperties(Obj, JsonToString(Root.ToSharedRef())));
			TestEqual(TEXT("Name unchanged (no field matched)"),
				Obj->StructProp.Name, FString(TEXT("original")));
			TestEqual(TEXT("Items unchanged (no field matched)"),
				Obj->StructProp.Items, TArray<int32>{1, 2, 3});
		});

		// ---------------------------------------------------------------------
		// TArray container notifications
		// ---------------------------------------------------------------------

		It(TEXT("emits ArrayAdd when appending to an array."), [this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {1, 2, 3, 4})));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 2, 3, 4});
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("ActiveProperty"), Evt.ActivePropertyName, FString(TEXT("IntArray")));
			TestEqual(TEXT("ElementIndex"), Evt.ElementIndices.FindRef(TEXT("IntArray")), 3);
		});

		It(TEXT("emits ArrayAdd with correct index when inserting mid-array."), [this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 3};

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {1, 2, 3})));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 2, 3});
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("ElementIndex"), Evt.ElementIndices.FindRef(TEXT("IntArray")), 1);
		});

		It(TEXT("emits ArrayRemove with correct index."), [this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {1, 3})));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 3});
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ArrayRemove);
			TestEqual(TEXT("ElementIndex"), Evt.ElementIndices.FindRef(TEXT("IntArray")), 1);
		});

		It(TEXT("emits ArrayClear when array is emptied."), [this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {})));

			TestTrue(TEXT("Array empty"), Obj->IntArray.IsEmpty());
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ArrayClear);
		});

		It(TEXT("emits ValueSet with element index when an array element changes."),
			[this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {1, 99, 3})));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 99, 3});
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ValueSet);
			TestEqual(TEXT("ElementIndex"), Evt.ElementIndices.FindRef(TEXT("IntArray")), 1);
		});

		It(TEXT("emits no notifications when array contents are unchanged."),
			[this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {1, 2, 3})));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 2, 3});
			TestEqual(TEXT("No notifications"), Obj->RecordedNotifications.Num(), 0);
		});

		It(TEXT("emits N ArrayAdd notifications when appending multiple elements."),
			[this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1};

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {1, 2, 3})));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 2, 3});
			if (!TestEqual(TEXT("Two notifications"), Obj->RecordedNotifications.Num(), 2)) return;

			TestEqual(TEXT("First ChangeType"), Obj->RecordedNotifications[0].ChangeType,
				EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("First index"), Obj->RecordedNotifications[0].ElementIndices.FindRef(TEXT("IntArray")), 1);
			TestEqual(TEXT("Second ChangeType"), Obj->RecordedNotifications[1].ChangeType,
				EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("Second index"), Obj->RecordedNotifications[1].ElementIndices.FindRef(TEXT("IntArray")), 2);
		});

		It(TEXT("emits N ArrayAdd notifications when inserting multiple elements mid-array."),
			[this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 4};

			// Insert 2 and 3 between 1 and 4.
			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {1, 2, 3, 4})));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 2, 3, 4});
			if (!TestEqual(TEXT("Two notifications"), Obj->RecordedNotifications.Num(), 2)) return;

			TestEqual(TEXT("First ChangeType"), Obj->RecordedNotifications[0].ChangeType,
				EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("First index"), Obj->RecordedNotifications[0].ElementIndices.FindRef(TEXT("IntArray")), 1);
			TestEqual(TEXT("Second ChangeType"), Obj->RecordedNotifications[1].ChangeType,
				EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("Second index"), Obj->RecordedNotifications[1].ElementIndices.FindRef(TEXT("IntArray")), 2);
		});

		It(TEXT("emits N ArrayRemove notifications when removing multiple elements."),
			[this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3, 4};

			// Remove 2 and 3.
			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {1, 4})));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 4});
			if (!TestEqual(TEXT("Two notifications"), Obj->RecordedNotifications.Num(), 2)) return;

			// Removes are processed right to left (highest old index first) so lower indices
			// are never shifted by earlier removals: old[2]=3 removed first, then old[1]=2.
			TestEqual(TEXT("First ChangeType"), Obj->RecordedNotifications[0].ChangeType,
				EPropertyChangeType::ArrayRemove);
			TestEqual(TEXT("First index"), Obj->RecordedNotifications[0].ElementIndices.FindRef(TEXT("IntArray")), 2);
			TestEqual(TEXT("Second ChangeType"), Obj->RecordedNotifications[1].ChangeType,
				EPropertyChangeType::ArrayRemove);
			TestEqual(TEXT("Second index"), Obj->RecordedNotifications[1].ElementIndices.FindRef(TEXT("IntArray")), 1);
		});

		It(TEXT("rejects mixed changes alongside multi-element size increase."),
			[this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			// Size increases by 2 but element 2 was also changed to 99 — old array is not a
			// subsequence of the new array, so this is not a pure add.
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set rejected"),
					UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {1, 99, 3, 4, 5})));
			});

			TestEqual(TEXT("Array unchanged"), Obj->IntArray, TArray<int32>{1, 2, 3});
			TestTrue(TEXT("Error message"), ExceptionHandler.GetException().Contains(TEXT("insertion points are ambiguous")));
		});

		It(TEXT("rejects ambiguous array add."), [this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2};

			// Size increases by 1 but both existing elements also changed — insertion is ambiguous.
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set rejected"),
					UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {9, 2, 9})));
			});

			TestEqual(TEXT("Array unchanged"), Obj->IntArray, TArray<int32>{1, 2});
			TestEqual(TEXT("No notifications"), Obj->RecordedNotifications.Num(), 0);
			TestTrue(TEXT("Error message"), ExceptionHandler.GetException().Contains(TEXT("insertion points are ambiguous")));
		});

		It(TEXT("rejects ambiguous array remove."), [this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			// Size decreases by 1 but a remaining element also changed — removed index is ambiguous.
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set rejected"),
					UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intArray"), {9, 3})));
			});

			TestEqual(TEXT("Array unchanged"), Obj->IntArray, TArray<int32>{1, 2, 3});
			TestEqual(TEXT("No notifications"), Obj->RecordedNotifications.Num(), 0);
			TestTrue(TEXT("Error message"), ExceptionHandler.GetException().Contains(TEXT("removed elements are ambiguous")));
		});

		// ---------------------------------------------------------------------
		// TMap container notifications
		// ---------------------------------------------------------------------

		It(TEXT("rejects mixed map change (adds and removes in same operation)."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);

			// Remove "B" and add "C" at the same time — ambiguous mixed change.
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 1);
			MapJson->SetNumberField(TEXT("C"), 3);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set rejected"), UToolsetLibrary::SetObjectProperties(
					Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));
			});

			TestEqual(TEXT("Map unchanged"), Obj->IntMap.Num(), 2);
			TestEqual(TEXT("No notifications"), Obj->RecordedNotifications.Num(), 0);
			TestTrue(TEXT("Error message"), ExceptionHandler.GetException().Contains(TEXT("ambiguous")));
		});

		It(TEXT("rejects map add when keys are also being removed (non-zero delta)."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);

			// Delta = +1, but B removed and C+D added — mixed change.
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 1);
			MapJson->SetNumberField(TEXT("C"), 3);
			MapJson->SetNumberField(TEXT("D"), 4);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set rejected"), UToolsetLibrary::SetObjectProperties(
					Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));
			});

			TestEqual(TEXT("Map unchanged"), Obj->IntMap.Num(), 2);
			TestEqual(TEXT("No notifications"), Obj->RecordedNotifications.Num(), 0);
			TestTrue(TEXT("Error message"), ExceptionHandler.GetException().Contains(TEXT("keys removed alongside")));
		});

		It(TEXT("rejects map remove when keys are also being added (non-zero delta)."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);
			Obj->IntMap.Add(TEXT("C"), 3);

			// Delta = -1, but B+C removed and D added — mixed change.
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 1);
			MapJson->SetNumberField(TEXT("D"), 4);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([&]()
			{
				TestFalse(TEXT("Set rejected"), UToolsetLibrary::SetObjectProperties(
					Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));
			});

			TestEqual(TEXT("Map unchanged"), Obj->IntMap.Num(), 3);
			TestEqual(TEXT("No notifications"), Obj->RecordedNotifications.Num(), 0);
			TestTrue(TEXT("Error message"), ExceptionHandler.GetException().Contains(TEXT("keys added alongside")));
		});

		It(TEXT("emits N ArrayAdd notifications when adding multiple map entries."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);

			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 1);
			MapJson->SetNumberField(TEXT("B"), 2);
			MapJson->SetNumberField(TEXT("C"), 3);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("Map size"), Obj->IntMap.Num(), 3);
			TestEqual(TEXT("Two notifications"), Obj->RecordedNotifications.Num(), 2);
			for (const FRecordedChangeEvent& Evt : Obj->RecordedNotifications)
			{
				TestEqual(TEXT("ChangeType ArrayAdd"), Evt.ChangeType, EPropertyChangeType::ArrayAdd);
				TestTrue(TEXT("LogicalIndex present"), Evt.ElementIndices.Contains(TEXT("IntMap")));
			}
		});

		It(TEXT("emits N ArrayRemove notifications when removing multiple map entries."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);
			Obj->IntMap.Add(TEXT("C"), 3);

			// Remove "B" and "C", keeping only "A".
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 1);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("Map size"), Obj->IntMap.Num(), 1);
			TestTrue(TEXT("A kept"), Obj->IntMap.Contains(TEXT("A")));
			TestEqual(TEXT("Two notifications"), Obj->RecordedNotifications.Num(), 2);
			for (const FRecordedChangeEvent& Evt : Obj->RecordedNotifications)
			{
				TestEqual(TEXT("ChangeType ArrayRemove"), Evt.ChangeType, EPropertyChangeType::ArrayRemove);
				TestTrue(TEXT("LogicalIndex present"), Evt.ElementIndices.Contains(TEXT("IntMap")));
			}
		});

		It(TEXT("emits ValueSet for kept keys with changed values (same-size map)."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);

			// Change only "A"'s value. "B" stays the same.
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 99);
			MapJson->SetNumberField(TEXT("B"), 2);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("A updated"), Obj->IntMap[TEXT("A")], 99);
			TestEqual(TEXT("B unchanged"), Obj->IntMap[TEXT("B")], 2);
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType ValueSet"), Evt.ChangeType, EPropertyChangeType::ValueSet);
			TestEqual(TEXT("ActiveProperty"), Evt.ActivePropertyName, FString(TEXT("IntMap")));
			TestTrue(TEXT("LogicalIndex present"), Evt.ElementIndices.Contains(TEXT("IntMap")));
		});

		It(TEXT("emits ValueSet for kept keys plus ArrayAdd for new keys."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);

			// Change "A"'s value AND add "C" in the same call.
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 99);
			MapJson->SetNumberField(TEXT("B"), 2);
			MapJson->SetNumberField(TEXT("C"), 3);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("A updated"), Obj->IntMap[TEXT("A")], 99);
			TestEqual(TEXT("C added"), Obj->IntMap[TEXT("C")], 3);
			if (!TestEqual(TEXT("Two notifications"), Obj->RecordedNotifications.Num(), 2)) return;

			const FRecordedChangeEvent* ValueSetEvt = Obj->RecordedNotifications.FindByPredicate(
				[](const FRecordedChangeEvent& E) { return E.ChangeType == EPropertyChangeType::ValueSet; });
			const FRecordedChangeEvent* AddEvt = Obj->RecordedNotifications.FindByPredicate(
				[](const FRecordedChangeEvent& E) { return E.ChangeType == EPropertyChangeType::ArrayAdd; });
			TestNotNull(TEXT("ValueSet for kept-key A"), ValueSetEvt);
			TestNotNull(TEXT("ArrayAdd for new key C"), AddEvt);
		});

		It(TEXT("emits multiple ValueSets for kept keys plus multiple ArrayAdds for new keys."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);

			// Change both A and B values AND add C and D in the same call.
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 99);
			MapJson->SetNumberField(TEXT("B"), 88);
			MapJson->SetNumberField(TEXT("C"), 3);
			MapJson->SetNumberField(TEXT("D"), 4);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("Map size"), Obj->IntMap.Num(), 4);
			TestEqual(TEXT("A updated"), Obj->IntMap[TEXT("A")], 99);
			TestEqual(TEXT("B updated"), Obj->IntMap[TEXT("B")], 88);
			TestEqual(TEXT("C added"), Obj->IntMap[TEXT("C")], 3);
			TestEqual(TEXT("D added"), Obj->IntMap[TEXT("D")], 4);
			if (!TestEqual(TEXT("Four notifications"), Obj->RecordedNotifications.Num(), 4)) return;

			int32 ValueSetCount = 0;
			int32 ArrayAddCount = 0;
			for (const FRecordedChangeEvent& Evt : Obj->RecordedNotifications)
			{
				TestTrue(TEXT("Has IntMap logical index"), Evt.ElementIndices.Contains(TEXT("IntMap")));
				if (Evt.ChangeType == EPropertyChangeType::ValueSet)
				{
					++ValueSetCount;
				}
				else if (Evt.ChangeType == EPropertyChangeType::ArrayAdd)
				{
					++ArrayAddCount;
				}
			}
			TestEqual(TEXT("Two ValueSets (kept keys A,B)"), ValueSetCount, 2);
			TestEqual(TEXT("Two ArrayAdds (new keys C,D)"), ArrayAddCount, 2);
		});

		It(TEXT("emits ArrayRemove for removed keys plus ValueSet for remaining keys."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);
			Obj->IntMap.Add(TEXT("C"), 3);

			// Remove "C" AND change "A"'s value.
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 99);
			MapJson->SetNumberField(TEXT("B"), 2);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("Map size"), Obj->IntMap.Num(), 2);
			TestEqual(TEXT("A updated"), Obj->IntMap[TEXT("A")], 99);
			TestFalse(TEXT("C removed"), Obj->IntMap.Contains(TEXT("C")));
			if (!TestEqual(TEXT("Two notifications"), Obj->RecordedNotifications.Num(), 2)) return;

			const FRecordedChangeEvent* RemoveEvt = Obj->RecordedNotifications.FindByPredicate(
				[](const FRecordedChangeEvent& E) { return E.ChangeType == EPropertyChangeType::ArrayRemove; });
			const FRecordedChangeEvent* ValueSetEvt = Obj->RecordedNotifications.FindByPredicate(
				[](const FRecordedChangeEvent& E) { return E.ChangeType == EPropertyChangeType::ValueSet; });
			TestNotNull(TEXT("ArrayRemove for removed key C"), RemoveEvt);
			TestNotNull(TEXT("ValueSet for kept-key A"), ValueSetEvt);
		});

		It(TEXT("recurses into struct-valued map entries for kept keys."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			FToolsetLibraryInnerStruct A;
			A.InnerItems = {1, 2, 3};
			Obj->StructMap.Add(TEXT("A"), A);

			// Same key, but its struct's InnerItems grows: {1,2,3} → {1,2,3,4}.
			TSharedPtr<FJsonObject> StructJson = MakeShared<FJsonObject>();
			StructJson->SetArrayField(TEXT("innerItems"), TArray<TSharedPtr<FJsonValue>>{
				MakeShared<FJsonValueNumber>(1), MakeShared<FJsonValueNumber>(2),
				MakeShared<FJsonValueNumber>(3), MakeShared<FJsonValueNumber>(4) });
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetObjectField(TEXT("A"), StructJson);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("structMap"), MapJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("InnerItems grew"), Obj->StructMap[TEXT("A")].InnerItems.Num(), 4);
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType ArrayAdd"), Evt.ChangeType, EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("ActiveProperty"), Evt.ActivePropertyName, FString(TEXT("InnerItems")));
			TestEqual(TEXT("MemberProperty"), Evt.MemberPropertyName, FString(TEXT("StructMap")));
			TestTrue(TEXT("StructMap index present"), Evt.ElementIndices.Contains(TEXT("StructMap")));
			TestEqual(TEXT("InnerItems insertion idx"), Evt.ElementIndices.FindRef(TEXT("InnerItems")), 3);
		});

		It(TEXT("emits per-field events for struct sibling properties."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->StructProp.Name = TEXT("");
			Obj->StructProp.Items = {1, 2, 3};

			TArray<TSharedPtr<FJsonValue>> NewItems = {
				MakeShared<FJsonValueNumber>(1), MakeShared<FJsonValueNumber>(2),
				MakeShared<FJsonValueNumber>(3), MakeShared<FJsonValueNumber>(4) };
			TSharedPtr<FJsonObject> StructJson = MakeShared<FJsonObject>();
			StructJson->SetStringField(TEXT("name"), TEXT("hello"));
			StructJson->SetArrayField(TEXT("items"), NewItems);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("structProp"), StructJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("Name set"), Obj->StructProp.Name, FString(TEXT("hello")));
			TestEqual(TEXT("Items size"), Obj->StructProp.Items.Num(), 4);
			TestEqual(TEXT("Two notifications"), Obj->RecordedNotifications.Num(), 2);

			// Events may arrive in any order (TMap iteration is unordered); find by property name.
			const FRecordedChangeEvent* NameEvt = Obj->RecordedNotifications.FindByPredicate(
				[](const FRecordedChangeEvent& E){ return E.ActivePropertyName == TEXT("Name"); });
			const FRecordedChangeEvent* ItemsEvt = Obj->RecordedNotifications.FindByPredicate(
				[](const FRecordedChangeEvent& E){ return E.ActivePropertyName == TEXT("Items"); });

			if (TestNotNull(TEXT("Name event"), NameEvt))
			{
				TestEqual(TEXT("Name ChangeType"), NameEvt->ChangeType, EPropertyChangeType::ValueSet);
				TestEqual(TEXT("Name MemberProperty"), NameEvt->MemberPropertyName, FString(TEXT("StructProp")));
			}
			if (TestNotNull(TEXT("Items event"), ItemsEvt))
			{
				TestEqual(TEXT("Items ChangeType"), ItemsEvt->ChangeType, EPropertyChangeType::ArrayAdd);
				TestEqual(TEXT("Items MemberProperty"), ItemsEvt->MemberPropertyName, FString(TEXT("StructProp")));
				TestEqual(TEXT("Items index"), ItemsEvt->ElementIndices.FindRef(TEXT("Items")), 3);
			}
		});

		It(TEXT("emits ArrayAdd with logical index when adding a map entry."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			// Start from empty so we know the added entry will be at logical index 0.

			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("OnlyKey"), 42);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestTrue(TEXT("Map entry present"), Obj->IntMap.Contains(TEXT("OnlyKey")));
			TestEqual(TEXT("Map value"), Obj->IntMap[TEXT("OnlyKey")], 42);
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("ActiveProperty"), Evt.ActivePropertyName, FString(TEXT("IntMap")));
			TestEqual(TEXT("LogicalIndex"), Evt.ElementIndices.FindRef(TEXT("IntMap")), 0);
		});

		It(TEXT("emits ArrayRemove with logical index when removing a map entry."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);

			// Remove "B" by importing a map with only "A".
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("A"), 1);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MapJson);

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestFalse(TEXT("B removed"), Obj->IntMap.Contains(TEXT("B")));
			TestTrue(TEXT("A kept"), Obj->IntMap.Contains(TEXT("A")));
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ArrayRemove);
			TestTrue(TEXT("LogicalIndex valid"), Evt.ElementIndices.Contains(TEXT("IntMap")));
		});

		It(TEXT("emits ArrayClear when a map is emptied."), [this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntMap.Add(TEXT("A"), 1);
			Obj->IntMap.Add(TEXT("B"), 2);

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("intMap"), MakeShared<FJsonObject>());

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestTrue(TEXT("Map empty"), Obj->IntMap.IsEmpty());
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			TestEqual(TEXT("ChangeType"),
				Obj->RecordedNotifications[0].ChangeType, EPropertyChangeType::ArrayClear);
		});

		It(TEXT("emits ArrayAdd with outer and inner indices for nested containers."),
			[this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->OuterArray.AddDefaulted();
			Obj->OuterArray[0].InnerItems = {1, 2, 3};

			TArray<TSharedPtr<FJsonValue>> InnerItems = {
				MakeShared<FJsonValueNumber>(1), MakeShared<FJsonValueNumber>(2),
				MakeShared<FJsonValueNumber>(3), MakeShared<FJsonValueNumber>(4) };
			TSharedPtr<FJsonObject> ElemJson = MakeShared<FJsonObject>();
			ElemJson->SetArrayField(TEXT("innerItems"), InnerItems);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetArrayField(TEXT("outerArray"),
				TArray<TSharedPtr<FJsonValue>>{ MakeShared<FJsonValueObject>(ElemJson) });

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("InnerItems size"), Obj->OuterArray[0].InnerItems.Num(), 4);
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("ActiveProperty"), Evt.ActivePropertyName, FString(TEXT("InnerItems")));
			TestEqual(TEXT("MemberProperty"), Evt.MemberPropertyName, FString(TEXT("OuterArray")));
			TestEqual(TEXT("OuterArray index"), Evt.ElementIndices.FindRef(TEXT("OuterArray")), 0);
			TestEqual(TEXT("InnerItems index"), Evt.ElementIndices.FindRef(TEXT("InnerItems")), 3);
		});

		// ---------------------------------------------------------------------
		// TSet container (routes through the array path)
		// ---------------------------------------------------------------------

		It(TEXT("emits ArrayAdd when appending to a TSet."), [this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntSet.Append({1, 2, 3});

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intSet"), {1, 2, 3, 4})));

			TestTrue(TEXT("Set contains 4"), Obj->IntSet.Contains(4));
			TestEqual(TEXT("Set size"), Obj->IntSet.Num(), 4);
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ArrayAdd);
			TestEqual(TEXT("ActiveProperty"), Evt.ActivePropertyName, FString(TEXT("IntSet")));
		});

		It(TEXT("emits ArrayRemove when removing from a TSet."), [this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntSet.Append({1, 2, 3});

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intSet"), {1, 3})));

			TestFalse(TEXT("2 removed"), Obj->IntSet.Contains(2));
			TestEqual(TEXT("Set size"), Obj->IntSet.Num(), 2);
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;

			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ArrayRemove);
			TestEqual(TEXT("ActiveProperty"), Evt.ActivePropertyName, FString(TEXT("IntSet")));
		});

		It(TEXT("emits ArrayClear when a TSet is emptied."), [this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntSet.Append({1, 2, 3});

			TestTrue(TEXT("Set succeeded"),
				UToolsetLibrary::SetObjectProperties(Obj, ArrayJson(TEXT("intSet"), {})));

			TestTrue(TEXT("Set empty"), Obj->IntSet.IsEmpty());
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;
			TestEqual(TEXT("ChangeType"),
				Obj->RecordedNotifications[0].ChangeType, EPropertyChangeType::ArrayClear);
		});

		// ---------------------------------------------------------------------
		// EBypassContainerCheck parameter
		// ---------------------------------------------------------------------

		It(TEXT("emits a single ValueSet for an array resize when BypassContainerCheck is Yes."),
			[this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			TArray<FName> SetNames;
			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, ArrayJson(TEXT("intArray"), {1, 2, 3, 4}), SetNames, EBypassContainerCheck::Yes));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 2, 3, 4});
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;
			TestEqual(TEXT("ChangeType"),
				Obj->RecordedNotifications[0].ChangeType, EPropertyChangeType::ValueSet);
		});

		It(TEXT("emits a single ValueSet for an array shrink when BypassContainerCheck is Yes."),
			[this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			TArray<FName> SetNames;
			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, ArrayJson(TEXT("intArray"), {1}), SetNames, EBypassContainerCheck::Yes));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1});
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;
			TestEqual(TEXT("ChangeType"),
				Obj->RecordedNotifications[0].ChangeType, EPropertyChangeType::ValueSet);
		});

		It(TEXT("preserves container diff behavior when BypassContainerCheck is No."),
			[this, ArrayJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};

			TArray<FName> SetNames;
			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, ArrayJson(TEXT("intArray"), {1, 2, 3, 4}), SetNames, EBypassContainerCheck::No));

			TestEqual(TEXT("Array value"), Obj->IntArray, TArray<int32>{1, 2, 3, 4});
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;
			TestEqual(TEXT("ChangeType"),
				Obj->RecordedNotifications[0].ChangeType, EPropertyChangeType::ArrayAdd);
		});

		// ---------------------------------------------------------------------
		// Struct properties whose ToolsetJson serialization is owned by a
		// registered FToolsetJsonConverter. Field-by-field recursion is wrong
		// for these — the JSON keys don't map to the struct's FProperty layout,
		// so the import must be delegated to the converter via JsonDataToProperty.
		// ---------------------------------------------------------------------

		// Builds {"location": {x, y, z}} — the FToolsetTransform JSON form that
		// FToolsetTransformConverter understands for FTransform properties.
		auto TransformLocationJson = [](double X, double Y, double Z) -> TSharedPtr<FJsonObject>
		{
			TSharedPtr<FJsonObject> Loc = MakeShared<FJsonObject>();
			Loc->SetNumberField(TEXT("x"), X);
			Loc->SetNumberField(TEXT("y"), Y);
			Loc->SetNumberField(TEXT("z"), Z);
			TSharedPtr<FJsonObject> Xform = MakeShared<FJsonObject>();
			Xform->SetObjectField(TEXT("location"), Loc);
			return Xform;
		};

		It(TEXT("delegates to the registered converter for a top-level FTransform property."),
			[this, TransformLocationJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetObjectField(TEXT("transformProp"), TransformLocationJson(10.0, 20.0, 30.0));

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			TestEqual(TEXT("Translation"), Obj->TransformProp.GetTranslation(), FVector(10.0, 20.0, 30.0));
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;
			TestEqual(TEXT("ChangeType"),
				Obj->RecordedNotifications[0].ChangeType, EPropertyChangeType::ValueSet);
			TestEqual(TEXT("ActiveProperty"),
				Obj->RecordedNotifications[0].ActivePropertyName, FString(TEXT("TransformProp")));
		});

		// Regression test for the silent failure on struct arrays whose element
		// type has a registered ToolsetJson converter. Pre-fix, HandleContainerElement
		// unconditionally recursed into the element via ImportStructFieldsWithNotify,
		// which tried to FindFProperty("location") on FTransform — no match, so the
		// write silently no-op'd while SetObjectProperties still returned true.
		It(TEXT("delegates to the registered converter for FTransform array elements."),
			[this, TransformLocationJson]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->TransformArray.AddDefaulted();

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetArrayField(TEXT("transformArray"),
				TArray<TSharedPtr<FJsonValue>>{
					MakeShared<FJsonValueObject>(TransformLocationJson(1.0, 2.0, 3.0)) });

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			if (!TestEqual(TEXT("Array size"), Obj->TransformArray.Num(), 1)) return;
			TestEqual(TEXT("Element translation"),
				Obj->TransformArray[0].GetTranslation(), FVector(1.0, 2.0, 3.0));

			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;
			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ValueSet);
			TestEqual(TEXT("MemberProperty"), Evt.MemberPropertyName, FString(TEXT("TransformArray")));
			TestEqual(TEXT("TransformArray index"),
				Evt.ElementIndices.FindRef(TEXT("TransformArray")), 0);
		});

		// Regression test for the field-name fall-through guard. When a struct
		// element's JSON has no key matching any FProperty on the element type
		// (e.g. UUserDefinedStruct members whose FNames are auto-suffixed), the
		// container handler must fall through to the ValueSet path so that
		// FJsonObjectConverter — which matches by GetAuthoredName — gets a chance
		// rather than ImportStructFieldsWithNotify silently no-op'ing.
		It(TEXT("falls through to ValueSet when struct array element JSON has no matching field."),
			[this]()
		{
			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->OuterArray.AddDefaulted();
			Obj->OuterArray[0].InnerItems = {1, 2, 3};

			// JSON key "unknownField" matches no FProperty on FToolsetLibraryInnerStruct.
			TSharedPtr<FJsonObject> ElemJson = MakeShared<FJsonObject>();
			ElemJson->SetNumberField(TEXT("unknownField"), 42);
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetArrayField(TEXT("outerArray"),
				TArray<TSharedPtr<FJsonValue>>{ MakeShared<FJsonValueObject>(ElemJson) });

			TestTrue(TEXT("Set succeeded"), UToolsetLibrary::SetObjectProperties(
				Obj, UE::ToolsetRegistry::Internal::JsonToString(Root.ToSharedRef())));

			// The element is reset by the ValueSet path (no per-field recursion),
			// so a single OuterArray-scoped ValueSet notification must be emitted —
			// not zero (silent failure) and not a recursed-into-InnerItems event.
			if (!TestEqual(TEXT("One notification"), Obj->RecordedNotifications.Num(), 1)) return;
			const FRecordedChangeEvent& Evt = Obj->RecordedNotifications[0];
			TestEqual(TEXT("ChangeType"), Evt.ChangeType, EPropertyChangeType::ValueSet);
			TestEqual(TEXT("MemberProperty"), Evt.MemberPropertyName, FString(TEXT("OuterArray")));
			TestTrue(TEXT("OuterArray index present"),
				Evt.ElementIndices.Contains(TEXT("OuterArray")));
		});
	});

	Describe(TEXT("SetObjectProperties block list"), [this]()
	{
		// RAII helper: swaps the configured block lists for the duration of a test
		// and restores them on scope exit so individual tests can't leak into each other.
		struct FScopedBlockLists
		{
			TArray<FString> SavedClasses;
			TArray<FString> SavedProperties;

			FScopedBlockLists(TArray<FString> Classes, TArray<FString> Properties)
			{
				UToolsetRegistrySettings* Mutable = GetMutableDefault<UToolsetRegistrySettings>();
				SavedClasses = MoveTemp(Mutable->SetObjectPropertiesBlockedClasses);
				SavedProperties = MoveTemp(Mutable->SetObjectPropertiesBlockedProperties);
				Mutable->SetObjectPropertiesBlockedClasses = MoveTemp(Classes);
				Mutable->SetObjectPropertiesBlockedProperties = MoveTemp(Properties);
			}

			~FScopedBlockLists()
			{
				UToolsetRegistrySettings* Mutable = GetMutableDefault<UToolsetRegistrySettings>();
				Mutable->SetObjectPropertiesBlockedClasses = MoveTemp(SavedClasses);
				Mutable->SetObjectPropertiesBlockedProperties = MoveTemp(SavedProperties);
			}
		};

		It(TEXT("rejects the call when the leaf class is on the block list."), [this]()
		{
			FScopedBlockLists Guard({TEXT("ToolsetJsonTestObject")}, {});

			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("visibleProperty"), true);

			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn(
				[&]()
				{
					TestFalse(TEXT("SetProp returns false"),
						UToolsetLibrary::SetObjectProperties(
							TestObject, JsonToString(JsonObject.ToSharedRef())));
				});
			TestFalse(TEXT("Property not modified"), TestObject->VisibleProperty);
			TestNotEqual(TEXT("Raised error"), *ExceptionHandler.GetException(), TEXT(""));
		});

		It(TEXT("rejects the call when a parent class is on the block list."), [this]()
		{
			// Anchored regex matches UObject's GetName() ("Object") but not the leaf
			// class name "ToolsetJsonTestObject" — proves the inheritance walk runs.
			FScopedBlockLists Guard({TEXT("/^Object$/")}, {});

			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("visibleProperty"), true);

			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn(
				[&]()
				{
					TestFalse(TEXT("SetProp returns false"),
						UToolsetLibrary::SetObjectProperties(
							TestObject, JsonToString(JsonObject.ToSharedRef())));
				});
			TestFalse(TEXT("Property not modified"), TestObject->VisibleProperty);
		});

		It(TEXT("skips a single property when ClassName.PropertyName is blocked."), [this]()
		{
			FScopedBlockLists Guard({}, {TEXT("ToolsetContainerTestObject.IntArray")});

			UToolsetContainerTestObject* Obj = NewObject<UToolsetContainerTestObject>();
			Obj->IntArray = {1, 2, 3};
			Obj->IntMap = {};

			TArray<TSharedPtr<FJsonValue>> ArrayItems;
			ArrayItems.Add(MakeShared<FJsonValueNumber>(9));
			TSharedPtr<FJsonObject> MapJson = MakeShared<FJsonObject>();
			MapJson->SetNumberField(TEXT("a"), 1);

			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			Root->SetArrayField(TEXT("intArray"), ArrayItems);
			Root->SetObjectField(TEXT("intMap"), MapJson);

			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn(
				[&]()
				{
					TestFalse(TEXT("SetProp returns false (partial failure)"),
						UToolsetLibrary::SetObjectProperties(
							Obj, JsonToString(Root.ToSharedRef())));
				});
			TestEqual(TEXT("Blocked property unchanged"), Obj->IntArray, TArray<int32>{1, 2, 3});
			TestEqual(TEXT("Unblocked property written"), Obj->IntMap.FindRef(TEXT("a")), 1);
		});

		It(TEXT("matches property patterns as case-insensitive substrings."), [this]()
		{
			// Bare pattern is matched as a case-insensitive substring against
			// "ClassName.PropertyName" — confirms the composed-string match path.
			FScopedBlockLists Guard({}, {TEXT(".visibleproperty")});

			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("visibleProperty"), true);

			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn(
				[&]()
				{
					TestFalse(TEXT("SetProp returns false"),
						UToolsetLibrary::SetObjectProperties(
							TestObject, JsonToString(JsonObject.ToSharedRef())));
				});
			TestFalse(TEXT("Property not modified"), TestObject->VisibleProperty);
		});

		It(TEXT("allows writes when block lists are empty."), [this]()
		{
			FScopedBlockLists Guard({}, {});

			UToolsetJsonTestObject* TestObject = NewObject<UToolsetJsonTestObject>();
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetBoolField(TEXT("visibleProperty"), true);

			TestTrue(TEXT("SetProp"), UToolsetLibrary::SetObjectProperties(
				TestObject, JsonToString(JsonObject.ToSharedRef())));
			TestTrue(TEXT("Property written"), TestObject->VisibleProperty);
		});
	});

	Describe(TEXT("GetDerivedClasses"), [this]()
	{
		It(TEXT("gets subclasses of a class."), [this]()
		{
			TArray<FSoftClassPath> Subclasses = UToolsetLibrary::GetDerivedClasses(
				UToolsetDefinition::StaticClass());
			TestTrue("Subclass", Subclasses.Contains(FSoftClassPath(UFakeToolset::StaticClass())));
		});

		It(TEXT("raises error with null object."), [this]()
		{
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn(
				[this]()
				{
					TArray<FSoftClassPath> DerivedClasses = UToolsetLibrary::GetDerivedClasses(nullptr);
					TestTrue(TEXT("No classes"), DerivedClasses.IsEmpty());
				});
			TestNotEqual(TEXT("Raised error"), *ExceptionHandler.GetException(), TEXT(""));
		});
	});

	Describe(TEXT("GetDerivedStructs"), [this]()
	{
		It(TEXT("gets substructs of a struct."), [this]()
		{
			TArray<UScriptStruct*> Substructs = UToolsetLibrary::GetDerivedStructs(
				FAnimNode_Base::StaticStruct());
			TestTrue("Subclass", Substructs.Contains(FAnimNode_Root::StaticStruct()));
		});

		It(TEXT("raises error with null object."), [this]()
		{
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn(
				[this]()
				{
					TArray<UScriptStruct*> DerivedStructs =
						UToolsetLibrary::GetDerivedStructs(nullptr);
					TestTrue(TEXT("No structs"), DerivedStructs.IsEmpty());
				});
			TestNotEqual(TEXT("Raised error"), *ExceptionHandler.GetException(), TEXT(""));
		});
	});
}


BEGIN_DEFINE_SPEC(
	FToolsetLibraryUndoTransactionSpec,
	"AI.ToolsetRegistry.ToolsetLibrary.UndoTransaction",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	UToolsetJsonTestObject* TestObject = nullptr;

	// Reset the global transaction buffer so each test starts without prior
	// undo entries leaking in from other tests.
	void ResetTransactions()
	{
		if (GEditor && GEditor->Trans)
		{
			GEditor->Trans->Reset(
				NSLOCTEXT(
					"FToolsetLibraryUndoTransactionSpec",
					"ResetReason",
					"Test setup"));
		}
	}

	// Begin/end a single transaction that flips TestObject->VisibleProperty
	// from false to true, committing it to the global undo stack.
	void RecordToggleTransaction()
	{
		GEditor->BeginTransaction(
			TEXT("FToolsetLibraryUndoTransactionSpec"),
			NSLOCTEXT(
				"FToolsetLibraryUndoTransactionSpec",
				"ToggleTransactionTitle",
				"Toggle VisibleProperty"),
			TestObject);
		TestObject->Modify();
		TestObject->VisibleProperty = true;
		GEditor->EndTransaction();
	}

END_DEFINE_SPEC(FToolsetLibraryUndoTransactionSpec)

void FToolsetLibraryUndoTransactionSpec::Define()
{
	if (!GEditor)
	{
		AddInfo(TEXT("Test skipped: No GEditor available."));
		return;
	}

	BeforeEach([this]
	{
		ResetTransactions();
		TestObject = NewObject<UToolsetJsonTestObject>(
			GetTransientPackage(),
			NAME_None,
			RF_Transient | RF_Transactional);
		TestObject->AddToRoot();
	});

	AfterEach([this]
	{
		if (TestObject)
		{
			TestObject->RemoveFromRoot();
			TestObject = nullptr;
		}
		ResetTransactions();
	});

	Describe("with no committed transaction on the stack", [this]
	{
		It("returns false and leaves the object unchanged", [this]
		{
			TestObject->VisibleProperty = true;
			const bool bUndone = UToolsetLibrary::UndoTransaction();
			TestFalse(TEXT("UndoTransaction return value"), bUndone);
			TestTrue(TEXT("VisibleProperty unchanged"), TestObject->VisibleProperty);
		});
	});

	Describe("with a committed transaction modifying an object", [this]
	{
		It("reverts the modification and returns true", [this]
		{
			RecordToggleTransaction();
			TestTrue(TEXT("VisibleProperty after transaction"), TestObject->VisibleProperty);

			const bool bUndone = UToolsetLibrary::UndoTransaction();

			TestTrue(TEXT("UndoTransaction return value"), bUndone);
			TestFalse(TEXT("VisibleProperty reverted"), TestObject->VisibleProperty);
		});

		It("with bCanRedo=false removes the transaction from the buffer", [this]
		{
			RecordToggleTransaction();
			const int32 QueueLengthBefore = GEditor->Trans->GetQueueLength();
			TestEqual(TEXT("Transaction is on the queue"), QueueLengthBefore, 1);

			UToolsetLibrary::UndoTransaction(/*bCanRedo=*/false);

			// With bCanRedo=false the transaction is popped off the queue
			// entirely so the user has no zombie redo entry to deal with.
			TestEqual(
				TEXT("Queue length after rollback"),
				GEditor->Trans->GetQueueLength(), 0);
		});

		It("with bCanRedo=true keeps the transaction available for redo", [this]
		{
			RecordToggleTransaction();

			UToolsetLibrary::UndoTransaction(/*bCanRedo=*/true);

			TestFalse(TEXT("VisibleProperty reverted"), TestObject->VisibleProperty);
			TestEqual(
				TEXT("Queue length unchanged"),
				GEditor->Trans->GetQueueLength(), 1);
			TestEqual(
				TEXT("Undo count incremented"),
				GEditor->Trans->GetUndoCount(), 1);
		});
	});

	Describe("GetActiveUndoCount", [this]
	{
		It("returns zero on an empty stack", [this]
		{
			TestEqual(
				TEXT("Active undo count when stack is empty"),
				UToolsetLibrary::GetActiveUndoCount(), 0);
		});

		It("returns the count of committed entries above the redo split", [this]
		{
			RecordToggleTransaction();
			TestEqual(
				TEXT("After one commit"),
				UToolsetLibrary::GetActiveUndoCount(), 1);

			RecordToggleTransaction();
			TestEqual(
				TEXT("After two commits"),
				UToolsetLibrary::GetActiveUndoCount(), 2);
		});

		It("excludes entries that have been moved into the redo half", [this]
		{
			RecordToggleTransaction();
			UToolsetLibrary::UndoTransaction(/*bCanRedo=*/true);

			// The entry is still on the queue (length 1) but sits in the
			// redo half (undo count 1); active count is the difference.
			TestEqual(
				TEXT("Active undo count after one undo with redo retained"),
				UToolsetLibrary::GetActiveUndoCount(), 0);
		});

		It("does not change when a begin/end pair makes no modifications", [this]
		{
			RecordToggleTransaction();
			const int32 Before = UToolsetLibrary::GetActiveUndoCount();

			// Open and close a transaction without modifying any UObjects.
			// UTransBuffer::End drops the transient record from the buffer,
			// so the active count must stay put.
			GEditor->BeginTransaction(
				TEXT("FToolsetLibraryUndoTransactionSpec"),
				NSLOCTEXT(
					"FToolsetLibraryUndoTransactionSpec",
					"EmptyTransactionTitle",
					"Empty transaction"),
				nullptr);
			GEditor->EndTransaction();

			TestEqual(
				TEXT("Active undo count unchanged after empty begin/end"),
				UToolsetLibrary::GetActiveUndoCount(), Before);
		});
	});

}

#endif
