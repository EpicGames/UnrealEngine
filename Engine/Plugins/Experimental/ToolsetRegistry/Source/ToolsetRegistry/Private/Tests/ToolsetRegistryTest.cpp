// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolsetRegistry.h"

#include "Misc/AutomationTest.h"
#include "UObject/StrongObjectPtr.h"

#include "ToolsetRegistryTestFlags.h"
#include "ToolsetRegistry/DelegateHandle.h"
#include "ToolsetRegistry/Toolset.h"
#include "FakeToolset.h"
#include "FakeConverter.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::ToolsetRegistry
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryToolsetDescriptorTestFromStringValidName,
		"AI.ToolsetRegistry.ToolsetDescriptor.FromString.ValidName",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryToolsetDescriptorTestFromStringValidName::RunTest(
		const FString& UnusedParameters)
	{
		// Test valid toolset and tool name.
		TValueOrError<FToolDescriptor, FString> Descriptor =
			FToolDescriptor::FromString(TEXT("ToolsetName.ToolName"));
		(void)TestTrue(TEXT("FromString_Valid"), Descriptor.HasValue());
		FToolDescriptor* Value = Descriptor.TryGetValue();
		if (TestNotNull(TEXT("FromString_Valid_FoundValue"), Value))
		{
			(void)TestEqual(TEXT("FromString_Valid_ToolsetName"),
				Value->ToolsetName, TEXT("ToolsetName"));
			(void)TestEqual(TEXT("FromString_Valid_ToolName"), Value->ToolName, TEXT("ToolName"));
		}
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryToolsetDescriptorTestFromStringInvalidToolsetName,
		"AI.ToolsetRegistry.ToolsetDescriptor.FromString.InvalidToolsetName",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryToolsetDescriptorTestFromStringInvalidToolsetName::RunTest(
		const FString& UnusedParameters)
	{
		// Test invalid toolset and tool name (missing dot).
		TValueOrError<FToolDescriptor, FString> Descriptor =
			FToolDescriptor::FromString(TEXT("InvalidToolNameWithoutDot"));
		(void)TestTrue(TEXT("FromString_Invalid_MissingDot"), Descriptor.HasError());
		(void)TestEqual(
			TEXT("FromString_Invalid_MissingDot_String"),
			Descriptor.GetError(),
			TEXT("Invalid tool name format. Expected format: ")
			TEXT("ToolsetName.ToolName, got InvalidToolNameWithoutDot"));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryToolsetDescriptorTestFromStringMissingToolsetName,
		"AI.ToolsetRegistry.ToolsetDescriptor.FromString.MissingToolsetName",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryToolsetDescriptorTestFromStringMissingToolsetName::RunTest(
		const FString& UnusedParameters)
	{
		// Test missing toolset name.
		TValueOrError<FToolDescriptor, FString> Descriptor =
			FToolDescriptor::FromString(TEXT(".ToolNameWithoutToolset"));
		(void)TestTrue(TEXT("FromString_Invalid_MissingToolsetName"), Descriptor.HasError());
		(void)TestEqual(
			TEXT("FromString_Invalid_MissingToolsetName_String"),
			Descriptor.GetError(),
			TEXT("Invalid tool name format. Expected format: ")
			TEXT("ToolsetName.ToolName, got .ToolNameWithoutToolset"));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryToolsetDescriptorTestFromStringMissingToolName,
		"AI.ToolsetRegistry.ToolsetDescriptor.FromString.MissingToolName",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryToolsetDescriptorTestFromStringMissingToolName::RunTest(
		const FString& UnusedParameters)
	{
		// Test missing tool name.
		TValueOrError<FToolDescriptor, FString> Descriptor =
			FToolDescriptor::FromString(TEXT("ToolsetWithoutToolname."));
		(void)TestTrue(TEXT("FromString_Invalid_MissingToolName"), Descriptor.HasError());
		(void)TestEqual(
			TEXT("FromString_Invalid_MissingToolName_String"),
			Descriptor.GetError(),
			TEXT("Invalid tool name format. Expected format: ")
			TEXT("ToolsetName.ToolName, got ToolsetWithoutToolname."));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryToolsetDescriptorTestFromStringMultipleDots,
		"AI.ToolsetRegistry.ToolsetDescriptor.FromString.MultipleDots",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryToolsetDescriptorTestFromStringMultipleDots::RunTest(
		const FString& UnusedParameters)
	{
		// Test valid toolset and tool name with multiple dots.
		TValueOrError<FToolDescriptor, FString> Descriptor =
			FToolDescriptor::FromString(TEXT("Complex.Toolset.Name.ToolName"));
		(void)TestTrue(TEXT("FromString_Valid_MultipleDots"), Descriptor.HasValue());
		FToolDescriptor* Value = Descriptor.TryGetValue();
		if (!Value)
		{
			return false;
		}
		(void)TestEqual(
			TEXT("FromString_Valid_MultipleDots_ToolsetName"),
			Descriptor.TryGetValue()->ToolsetName,
			TEXT("Complex.Toolset.Name"));
		(void)TestEqual(
			TEXT("FromString_Valid_MultipleDots_ToolName"),
			Descriptor.TryGetValue()->ToolName,
			TEXT("ToolName"));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestRegistrationSuccess,
		"AI.ToolsetRegistry.ToolsetRegistry.Registration.Success",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestRegistrationSuccess::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();

		// Test whether a valid toolset handler is accepted.
		TSharedPtr<FToolset> ValidToolsetHandler = MakeShared<FFakeToolset>(TEXT("Fake"));
		(void)TestTrue(
			TEXT("RegisterToolsetHandler_Valid"),
			Registry->RegisterToolset(ValidToolsetHandler));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestRegistrationDoubleRegistration,
		"AI.ToolsetRegistry.ToolsetRegistry.Registration.DoubleRegistration",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestRegistrationDoubleRegistration::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		TSharedPtr<FToolset> ValidToolsetHandler = MakeShared<FFakeToolset>(TEXT("Fake"));
		Registry->RegisterToolset(ValidToolsetHandler);

		// Test whether duplicate registration is rejected.
		AddExpectedError(TEXT("Toolset 'Fake' already registered.*"),
			EAutomationExpectedMessageFlags::Contains, 1, true);
		(void)TestFalse(
			TEXT("RegisterToolsetHandler_Duplicate"),
			Registry->RegisterToolset(ValidToolsetHandler));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestUnregistrationSuccess,
		"AI.ToolsetRegistry.ToolsetRegistry.Unregistration.Success",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestUnregistrationSuccess::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		TSharedPtr<FToolset> ValidToolsetHandler = MakeShared<FFakeToolset>(TEXT("Fake"));
		Registry->RegisterToolset(ValidToolsetHandler);

		// Test whether unregistration of a registered toolset handler is accepted.
		(void)TestTrue(
			TEXT("UnregisterToolsetHandler_Valid"),
			Registry->UnregisterToolset(ValidToolsetHandler));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestUnregistrationUnregisteredToolset,
		"AI.ToolsetRegistry.ToolsetRegistry.Unregistration.UnregisteredToolset",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestUnregistrationUnregisteredToolset::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		Registry->RegisterToolset(MakeShared<FFakeToolset>(TEXT("Fake")));

		// Test whether unregistration of an non-registered toolset handler is rejected.
		TSharedPtr<FToolset> UnregisteredToolsetHandler =
			MakeShared<FFakeToolset>(TEXT("AnotherFake"));
		AddExpectedError(TEXT("Toolset 'AnotherFake' not registered"));
		(void)TestFalse(
			TEXT("UnregisterToolsetHandler_NotRegistered"),
			Registry->UnregisterToolset(UnregisteredToolsetHandler));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestUnregistrationDoubleUnregistration,
		"AI.ToolsetRegistry.ToolsetRegistry.Unregistration.DoubleUnregistration",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestUnregistrationDoubleUnregistration::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		TSharedPtr<FToolset> ValidToolsetHandler = MakeShared<FFakeToolset>(TEXT("Fake"));
		Registry->RegisterToolset(ValidToolsetHandler);
		Registry->UnregisterToolset(ValidToolsetHandler);

		// Test whether unregistration of an already unregistered toolset handler is rejected.
		AddExpectedError(TEXT("Toolset 'Fake' not registered"));
		(void)TestFalse(
			TEXT("UnregisterToolsetHandler_AlreadyUnregistered"),
			Registry->UnregisterToolset(ValidToolsetHandler));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestFindSuccess,
		"AI.ToolsetRegistry.ToolsetRegistry.Find.Success",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestFindSuccess::RunTest(const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		TSharedPtr<FToolset> Fake = MakeShared<FFakeToolset>(TEXT("Fake"));
		TSharedPtr<FToolset> AnotherFake = MakeShared<FFakeToolset>(TEXT("AnotherFake"));
		Registry->RegisterToolset(Fake);
		Registry->RegisterToolset(AnotherFake);

		// Test finding registered toolset handlers.
		TSharedPtr<FToolset> FoundToolsetHandler = Registry->Find(TEXT("Fake"));
		(void)TestTrue(
			TEXT("FindToolsetHandler_Registered_NotNull"), FoundToolsetHandler.IsValid());
		if (!FoundToolsetHandler)
		{
			return false;
		}
		(void)TestEqual(TEXT("FindToolsetHandler_Registered_Correct"), FoundToolsetHandler, Fake);

		FoundToolsetHandler = Registry->Find(TEXT("AnotherFake"));
		(void)TestTrue(TEXT("FindToolsetHandler_AnotherRegistered_NotNull"),
			FoundToolsetHandler.IsValid());
		if (!FoundToolsetHandler)
		{
			return false;
		}
		(void)TestEqual(
			TEXT("FindToolsetHandler_AnotherRegistered_Correct"),
			FoundToolsetHandler, AnotherFake);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestFindNonexistentToolsetLogError,
		"AI.ToolsetRegistry.ToolsetRegistry.Find.NonexistentToolset.LogError",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestFindNonexistentToolsetLogError::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		TSharedPtr<FToolset> Fake = MakeShared<FFakeToolset>(TEXT("Fake"));
		Registry->RegisterToolset(Fake);

		// Test finding an unregistered toolset handler.
		AddExpectedError(TEXT("Toolset 'NonExistentToolset' not found"));
		TSharedPtr<FToolset> FoundToolsetHandler =
			Registry->Find(TEXT("NonExistentToolset"), true);
		(void)TestFalse(TEXT("FindToolsetHandler_Unregistered"), FoundToolsetHandler.IsValid());

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestFindNonexistentToolsetReturnErrorMessage,
		"AI.ToolsetRegistry.ToolsetRegistry.Find.NonexistentToolset.ReturnErrorMessage",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestFindNonexistentToolsetReturnErrorMessage::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		TSharedPtr<FToolset> Fake = MakeShared<FFakeToolset>(TEXT("Fake"));
		Registry->RegisterToolset(Fake);

		// Test finding an unregistered toolset handler with error message.
		FString ErrorMessage;
		TSharedPtr<FToolset> FoundToolsetHandler =
			Registry->Find(TEXT("NonExistentToolset"), false, &ErrorMessage);
		(void)TestFalse(
			TEXT("FindToolsetHandler_UnregisteredWithErrorMessage_Found"),
			FoundToolsetHandler.IsValid());
		(void)TestEqual(
			TEXT("FindToolsetHandler_UnregisteredWithErrorMessage_ErrorMessage"),
			ErrorMessage,
			TEXT("Toolset 'NonExistentToolset' not found"));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestForEachToolsetSuccess,
		"AI.ToolsetRegistry.ToolsetRegistry.ForEachToolset.Success",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestForEachToolsetSuccess::RunTest(const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		TSharedPtr<FToolset> Fake = MakeShared<FFakeToolset>(TEXT("Fake"));
		TSharedPtr<FToolset> AnotherFake = MakeShared<FFakeToolset>(TEXT("AnotherFake"));
		Registry->RegisterToolset(Fake);
		Registry->RegisterToolset(AnotherFake);

		TArray<FString> VisitedNames;
		Registry->ForEachToolset([&VisitedNames](const FString& Name, const FToolset&)
			{
				VisitedNames.Add(Name);
			});

		FString VisitedNamesString = FString::Join(VisitedNames, TEXT(", "));
		for (const auto& Toolset : TArray<TSharedPtr<FToolset>>({ Fake, AnotherFake }))
		{
			(void)TestTrue(
				FString::Printf(
					TEXT("VisitedNames (%s) contains %s"), *VisitedNamesString, *Toolset->GetToolsetName()),
				VisitedNames.Contains(Toolset->GetToolsetName()));
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestExecuteToolSuccess,
		"AI.ToolsetRegistry.ToolsetRegistry.ExecuteTool.Success",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestExecuteToolSuccess::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();

		// Register a valid toolset handler.
		TSharedPtr<FToolset> ValidToolsetHandler = MakeShared<FFakeToolset>(TEXT("Fake"));
		(void)TestTrue(
			TEXT("RegisterToolsetHandler_Valid"),
			Registry->RegisterToolset(ValidToolsetHandler));

		FString InputArguments(TEXT(R"({\"input\": \"value\"})"));

		// Test execution of a tool from the registered toolset handler.
		Registry->ExecuteTool(TEXT("Fake.SomeTool"), InputArguments)
			.Next([this](TValueOrError<FString, FString> Result)
				{
					(void)TestTrue(TEXT("ExecuteTool_SuccessValue"), Result.HasValue());
					(void)TestEqual(
						TEXT("ExecuteTool_ResultJson"),
						Result.StealValue(),
						FFakeToolset::SUCCESSFUL_RESULT);
				})
			.Wait();

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestExecuteToolNonexistantTool,
		"AI.ToolsetRegistry.ToolsetRegistry.ExecuteTool.NonexistantTool",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestExecuteToolNonexistantTool::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();

		// Register a valid toolset handler.
		TSharedPtr<FToolset> ValidToolsetHandler = MakeShared<FFakeToolset>(TEXT("Fake"));
		(void)TestTrue(
			TEXT("RegisterToolsetHandler_Valid"),
			Registry->RegisterToolset(ValidToolsetHandler));

		// Test execution of an invalid tool from the registered toolset handler.
		Registry->ExecuteTool(TEXT("Fake.SomeNonexistantTool"), FString())
			.Next([this](TValueOrError<FString, FString> Result)
				{
					(void)TestTrue(TEXT("NonexistantToolset_HasError"), Result.HasError());
					(void)TestEqual(
						TEXT("ExecuteTool_InvalidTool_Message"),
						*Result.GetError(),
						TEXT("Tool not found"));
				})
			.Wait();

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestExecuteToolNonexistantToolset,
		"AI.ToolsetRegistry.ToolsetRegistry.ExecuteTool.NonexistantToolset",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestExecuteToolNonexistantToolset::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();

		// Register a valid toolset handler.
		TSharedPtr<FToolset> ValidToolsetHandler = MakeShared<FFakeToolset>(TEXT("Fake"));
		(void)TestTrue(
			TEXT("RegisterToolsetHandler_Valid"),
			Registry->RegisterToolset(ValidToolsetHandler));

		// Test execution of an invalid toolset.
		AddExpectedError(TEXT("Toolset 'InvalidTool' not found"));
		Registry->ExecuteTool(TEXT("InvalidTool.SolveHaltingProblem"), FString())
			.Next([this](TValueOrError<FString, FString> Result)
				{
					(void)TestTrue(TEXT("ExecuteTool_InvalidToolset_HasError"), Result.HasError());
					(void)TestEqual(
						TEXT("ExecuteTool_InvalidToolset_Message"),
						*Result.GetError(),
						TEXT("Toolset 'InvalidTool' not found"));
				})
			.Wait();

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestExecuteToolBadToolnameFormat,
		"AI.ToolsetRegistry.ToolsetRegistry.ExecuteTool.BadToolnameFormat",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestExecuteToolBadToolnameFormat::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();

		// Register a valid toolset handler.
		TSharedPtr<FToolset> ValidToolsetHandler = MakeShared<FFakeToolset>(TEXT("Fake"));
		(void)TestTrue(
			TEXT("RegisterToolsetHandler_Valid"),
			Registry->RegisterToolset(ValidToolsetHandler));

		// Test execution of an incorrectly formatted tool name.
		AddExpectedError(TEXT("Invalid tool name format. Expected format: ToolsetName.ToolName.*"),
			EAutomationExpectedMessageFlags::Contains, 1, true);
		Registry->ExecuteTool(TEXT("ToolCallThatDoesNotHaveADot"), FString())
			.Next([this](TValueOrError<FString, FString> Result)
				{
					(void)TestTrue(
						TEXT("ExecuteTool_InvalidToolName_HasError"), Result.HasError());
					(void)TestEqual(
						TEXT("ExecuteTool_InvalidToolName_ResultJson"),
						*Result.GetError(),
						TEXT("Invalid tool name format. Expected format: ToolsetName.ToolName"));
				})
			.Wait();


		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestGetToolsetJsonSchemasEmpty,
		"AI.ToolsetRegistry.ToolsetRegistry.GetToolsetJsonSchemas.Empty",
		ToolsetRegistryTest::Flags);
	bool FToolsetRegistryTestGetToolsetJsonSchemasEmpty::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		// Test getting toolset JSON schemas when no toolsets are registered.
		FString Schemas = Registry->GetToolsetJsonSchemas();
		TestEqual(TEXT("GetToolsetJsonSchemas_Empty"), Schemas, TEXT("[]"));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestGetToolsetJsonSchemasSingleTool,
		"AI.ToolsetRegistry.ToolsetRegistry.GetToolsetJsonSchemas.SingleTool",
		ToolsetRegistryTest::Flags);
	bool FToolsetRegistryTestGetToolsetJsonSchemasSingleTool::RunTest(
		const FString& UnusedParameters)
	{
		auto Registry = MakeUnique<FToolsetRegistry>();
		auto ToolsetHandler = MakeShared<FFakeToolset>(TEXT("Fake"));
		TestTrue(
			TEXT("RegisterToolsetHandler_Valid"),
			Registry->RegisterToolset(ToolsetHandler));
		// Test getting toolset JSON schemas when one toolset is registered.
		TestEqual(
			TEXT("GetToolsetJsonSchemas_Single_ToolsetName"),
			Registry->GetToolsetJsonSchemas(),
			FString(TEXT(R"json([{"name":"Fake","tools":[{"name":"Fake.SomeTool"}]}])json")));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestGetToolsetJsonSchemasMultipleTools,
		"AI.ToolsetRegistry.ToolsetRegistry.GetToolsetJsonSchemas.MultipleTools",
		ToolsetRegistryTest::Flags);
	bool FToolsetRegistryTestGetToolsetJsonSchemasMultipleTools::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		// Register multiple valid toolset handlers.
		auto ToolsetHandler1 = MakeShared<FFakeToolset>(TEXT("Fake"));
		(void)TestTrue(
			TEXT("RegisterToolsetHandler1_Valid"),
			Registry->RegisterToolset(ToolsetHandler1));
		ToolsetHandler1->AddFakeToolCall(
			TEXT("AnotherTool"),
			MakeFulfilledPromise<TValueOrError<FString, FString>>(
				MakeValue(FFakeToolset::SUCCESSFUL_RESULT)));
		auto ToolsetHandler2 = MakeShared<FFakeToolset>(TEXT("AnotherFake"));
		(void)TestTrue(
			TEXT("RegisterToolsetHandler2_Valid"),
			Registry->RegisterToolset(ToolsetHandler2));
		ToolsetHandler2->AddFakeToolCall(
			TEXT("AnotherTool"),
			MakeFulfilledPromise<TValueOrError<FString, FString>>(
				MakeValue(FFakeToolset::SUCCESSFUL_RESULT)));
		// Test getting toolset JSON schemas when multiple toolsets are registered.
		TestTrue(
			TEXT("GetToolsetJsonSchemas_FakeToolset"),
			Registry->GetToolsetJsonSchemas().Contains(
				TEXT(R"json({"name":"Fake","tools":)json")));
		TestTrue(
			TEXT("GetToolsetJsonSchemas_AnotherFakeToolset"),
			Registry->GetToolsetJsonSchemas().Contains(
				TEXT(R"json({"name":"AnotherFake","tools":)json")));
		TestTrue(
			TEXT("GetToolsetJsonSchemas_Fake_SomeTool"),
			Registry->GetToolsetJsonSchemas().Contains(
				TEXT(R"json("Fake.SomeTool")json")));
		TestTrue(
			TEXT("GetToolsetJsonSchemas_Fake_SomeTool"),
			Registry->GetToolsetJsonSchemas().Contains(
				TEXT(R"json("Fake.AnotherTool")json")));
		TestTrue(
			TEXT("GetToolsetJsonSchemas_Fake_SomeTool"),
			Registry->GetToolsetJsonSchemas().Contains(
				TEXT(R"json("AnotherFake.SomeTool")json")));
		TestTrue(
			TEXT("GetToolsetJsonSchemas_Fake_SomeTool"),
			Registry->GetToolsetJsonSchemas().Contains(
				TEXT(R"json("AnotherFake.AnotherTool")json")));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestConverterRegistrationSuccess,
		"AI.ToolsetRegistry.ToolsetConverter.Registration.Success",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestConverterRegistrationSuccess::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();

		// Test whether a valid toolset converter is accepted.
		TSharedPtr<FFakeConverter> ValidConverter = MakeShared<FFakeConverter>();
		(void)TestTrue(
			TEXT("RegisterConverter_Valid"),
			Registry->RegisterConverter(ValidConverter));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestConverterUnregistrationSuccess,
		"AI.ToolsetRegistry.ToolsetConverter.Unregistration.Success",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestConverterUnregistrationSuccess::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();
		TSharedPtr<FFakeConverter> ValidConverter = MakeShared<FFakeConverter>();
		Registry->RegisterConverter(ValidConverter);

		// Test whether unregistration of a registered toolset handler is accepted.
		(void)TestTrue(
			TEXT("UnregisterConverter_Valid"),
			Registry->UnregisterConverter(ValidConverter));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestConverterGetConverterSuccess,
		"AI.ToolsetRegistry.ToolsetConverter.GetConverter.Success",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestConverterGetConverterSuccess::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();

		// Test whether converter lookup succeeds.
		TSharedPtr<FToolsetJsonConverter> ValidConverter = MakeShared<FFakeConverter>();
		Registry->RegisterConverter(ValidConverter);

		UStruct* Struct = FFakeConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestFloat");

		(void)TestEqual(
			TEXT("GetConverter_Success"),
			Registry->GetConverterForProperty(Prop), ValidConverter);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FToolsetRegistryTestConverterGetConverterFailure,
		"AI.ToolsetRegistry.ToolsetConverter.GetConverter.Failure",
		ToolsetRegistryTest::Flags);

	bool FToolsetRegistryTestConverterGetConverterFailure::RunTest(
		const FString& UnusedParameters)
	{
		TUniquePtr<FToolsetRegistry> Registry = MakeUnique<FToolsetRegistry>();

		// Test whether converter lookup fails.
		TSharedPtr<FToolsetJsonConverter> ValidConverter = MakeShared<FFakeConverter>();
		Registry->RegisterConverter(ValidConverter);

		UStruct* Struct = FFakeConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestString");

		(void)TestEqual(
			TEXT("GetConverter_Failed"),
			Registry->GetConverterForProperty(Prop), TSharedPtr<FToolsetJsonConverter>());

		return true;
	}

	BEGIN_DEFINE_SPEC(
		FToolsetRegistryFilterSpec,
		"AI.ToolsetRegistry.ToolsetRegistry.Filters",
		ToolsetRegistryTest::Flags)
		TUniquePtr<FToolsetRegistry> Registry;
		TSharedPtr<FFakeToolset> Fake;
		TSharedPtr<FFakeToolset> Other;

		void ExpectToolSucceeds(const FString& FullToolName)
		{
			Registry->ExecuteTool(FullToolName, FString())
				.Next([this, FullToolName](TValueOrError<FString, FString> Result)
				{
					TestTrue(FString::Printf(TEXT("%s HasValue"), *FullToolName), Result.HasValue());
				})
				.Wait();
		}

		void ExpectToolFails(const FString& FullToolName)
		{
			Registry->ExecuteTool(FullToolName, FString())
				.Next([this, FullToolName](TValueOrError<FString, FString> Result)
				{
					TestTrue(FString::Printf(TEXT("%s HasError"), *FullToolName), Result.HasError());
				})
				.Wait();
		}
	END_DEFINE_SPEC(FToolsetRegistryFilterSpec)

	void FToolsetRegistryFilterSpec::Define()
	{
		BeforeEach([this]()
		{
			Registry = MakeUnique<FToolsetRegistry>();
			Fake = MakeShared<FFakeToolset>(TEXT("Fake"));
			Other = MakeShared<FFakeToolset>(TEXT("Other"));
			Registry->RegisterToolset(Fake);
			Registry->RegisterToolset(Other);
		});

		Describe("BlockList", [this]()
		{
			for (const FString& Pattern : TArray<FString>{TEXT("Fake"), TEXT("/^Fake$/")})
			{
				Describe(Pattern, [this, Pattern]()
				{
					BeforeEach([this, Pattern]()
					{
						Registry->AddBlockedName(Pattern);
					});

					It("GetBlockedNames contains the blocked pattern", [this, Pattern]()
					{
						const TArray<FString>& BlockedNames = Registry->GetBlockedNames();
						TestTrue(
							FString::Printf(TEXT("GetBlockedNames (%s) contains %s"),
								*FString::Join(BlockedNames, TEXT(", ")), *Pattern),
							BlockedNames.Contains(Pattern));
					});

					Describe("Remove", [this, Pattern]()
					{
						BeforeEach([this, Pattern]()
						{
							Registry->RemoveBlockedName(Pattern);
						});

						It("GetBlockedNames no longer contains the removed pattern",
							[this, Pattern]()
						{
							TestFalse(
								FString::Printf(TEXT("GetBlockedNames_NotContains_%s"), *Pattern),
								Registry->GetBlockedNames().Contains(Pattern));
						});

						It("Find works again after block is removed", [this]()
						{
							TestTrue(TEXT("Find_Unblocked"), Registry->Find(TEXT("Fake")).IsValid());
						});
					});
				});
			}
		});

		Describe("BlockList - per tool", [this]()
		{
			BeforeEach([this]()
			{
				Fake->AddFakeToolCall(
					TEXT("OtherTool"),
					MakeFulfilledPromise<TValueOrError<FString, FString>>(
						MakeValue(FFakeToolset::SUCCESSFUL_RESULT)));
				Registry->AddBlockedName(TEXT("Fake.SomeTool"));
			});

			It("Find still works when only a tool is blocked", [this]()
			{
				TestTrue(TEXT("Find_ToolBlocked_Valid"), Registry->Find(TEXT("Fake")).IsValid());
			});

			It("ExecuteTool returns error for blocked tool and succeeds for unblocked tool", [this]()
			{
				ExpectToolFails(TEXT("Fake.SomeTool"));
				ExpectToolSucceeds(TEXT("Fake.OtherTool"));
			});
		});

		Describe("AllowList - per tool", [this]()
		{
			BeforeEach([this]()
			{
				Fake->AddFakeToolCall(
					TEXT("OtherTool"),
					MakeFulfilledPromise<TValueOrError<FString, FString>>(
						MakeValue(FFakeToolset::SUCCESSFUL_RESULT)));
				Registry->AddAllowedName(TEXT("/^Fake$/"));
				Registry->AddAllowedName(TEXT("Fake.SomeTool"));
			});

			It("Find still works when only a tool is allowed", [this]()
			{
				TestTrue(TEXT("Find_ToolAllowed_Valid"), Registry->Find(TEXT("Fake")).IsValid());
			});

			It("ExecuteTool succeeds for allowed tool and returns error for non-allowed tool",
				[this]()
			{
				ExpectToolSucceeds(TEXT("Fake.SomeTool"));
				ExpectToolFails(TEXT("Fake.OtherTool"));
			});
		});

		Describe("AllowList", [this]()
		{
			for (const FString& Pattern : TArray<FString>{TEXT("Fake"), TEXT("/^Fake$/")})
			{
				Describe(Pattern, [this, Pattern]()
				{
					BeforeEach([this, Pattern]()
					{
						Registry->AddAllowedName(Pattern);
					});

					It("GetAllowedNames contains the allowed pattern", [this, Pattern]()
					{
						const TArray<FString>& AllowedNames = Registry->GetAllowedNames();
						TestTrue(
							FString::Printf(TEXT("GetAllowedNames (%s) contains %s"),
								*FString::Join(AllowedNames, TEXT(", ")), *Pattern),
							AllowedNames.Contains(Pattern));
					});

					Describe("Remove", [this, Pattern]()
					{
						BeforeEach([this, Pattern]()
						{
							Registry->RemoveAllowedName(Pattern);
						});

						It("GetAllowedNames no longer contains the removed pattern",
							[this, Pattern]()
						{
							TestFalse(
								FString::Printf(TEXT("GetAllowedNames_NotContains_%s"), *Pattern),
								Registry->GetAllowedNames().Contains(Pattern));
						});

						It("All toolsets re-enabled when allow list is cleared", [this]()
						{
							TestTrue(TEXT("Find_Fake_Reenabled"), Registry->Find(TEXT("Fake")).IsValid());
							TestTrue(TEXT("Find_Other_Reenabled"), Registry->Find(TEXT("Other")).IsValid());
						});
					});
				});
			}
		});
	}

	BEGIN_DEFINE_SPEC(
		FToolsetRegistryOnToolsetRegisteredCallbackSpec,
		"AI.ToolsetRegistry.ToolsetRegistry.OnToolsetRegisteredCallback",
		ToolsetRegistryTest::Flags)
		TUniquePtr<FToolsetRegistry> Registry;
	END_DEFINE_SPEC(FToolsetRegistryOnToolsetRegisteredCallbackSpec)

		void FToolsetRegistryOnToolsetRegisteredCallbackSpec::Define()
	{
		BeforeEach([this]()
		{
			Registry = MakeUnique<FToolsetRegistry>();
		});

		AfterEach([this]()
		{
			Registry.Reset();
		});

		Describe("RegisterOnToolsetRegisteredCallback", [this]()
		{
			It("calls registered callbacks when a toolset is registered", [this]()
			{
				int32 CallCount = 0;
				auto RaiiHandle = FDelegateHandleRaii::Create(
					Registry->OnToolsetRegistered(),
					Registry->OnToolsetRegistered().AddLambda([&]() { ++CallCount; }));

				TestEqual(TEXT("CallCount_Initial"), CallCount, 0);

				(void)Registry->RegisterToolset(MakeShared<FFakeToolset>(TEXT("Fake")));

				TestEqual(TEXT("CallCount_AfterRegisterToolset"), CallCount, 1);
			});

			It("calls registered callbacks when a toolset is unregistered", [this]()
			{
				int32 CallCount = 0;
				auto RaiiHandle = FDelegateHandleRaii::Create(
					Registry->OnToolsetRegistered(),
					Registry->OnToolsetRegistered().AddLambda([&]() { ++CallCount; }));

				TSharedPtr<FToolset> Toolset =
					MakeShared<FFakeToolset>(TEXT("Fake"));
				(void)Registry->RegisterToolset(Toolset);
				TestEqual(TEXT("CallCount_AfterRegisterToolset"), CallCount, 1);

				(void)Registry->UnregisterToolset(Toolset);
				TestEqual(TEXT("CallCount_AfterUnregisterToolset"), CallCount, 2);
			});
		});
	}
}

#endif
