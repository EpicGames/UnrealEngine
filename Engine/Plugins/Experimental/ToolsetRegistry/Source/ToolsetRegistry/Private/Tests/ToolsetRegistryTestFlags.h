// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace ToolsetRegistryTest
{
	// Default flags to use for tests.
	const auto Flags =
		EAutomationTestFlags::EditorContext |
		EAutomationTestFlags::ProductFilter |
		EAutomationTestFlags::CriticalPriority;
}  // namespace ToolsetRegistryTest

#endif  // WITH_DEV_AUTOMATION_TESTS
