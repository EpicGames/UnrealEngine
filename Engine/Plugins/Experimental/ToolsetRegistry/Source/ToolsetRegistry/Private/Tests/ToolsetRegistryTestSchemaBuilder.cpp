// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistryTestSchemaBuilder.h"

#include "CoreMinimal.h"
#include "JsonDomBuilder.h"

FJsonDomBuilder::FObject MakeSchemaBuilder(
	const FString& Name, const FString& Version, const FString& Description)
{
	FJsonDomBuilder::FObject SchemaBuilder;
	SchemaBuilder.Set(TEXT("name"), Name);
	SchemaBuilder.Set(TEXT("version"), Version);
	SchemaBuilder.Set(TEXT("description"), Description);
	return SchemaBuilder;
}

FJsonDomBuilder::FArray AddToolSchema(
	FJsonDomBuilder::FObject& Builder, FJsonDomBuilder::FArray& ToolsArray,
	FJsonDomBuilder::FObject& ToolEntry)
{
	ToolsArray.Add(ToolEntry);
	Builder.Set(TEXT("tools"), ToolsArray);
	return ToolsArray;
}

FJsonDomBuilder::FObject MakeToolSchemaEntry(
	const FString& Name, const FString& Description,
	FJsonDomBuilder::FObject& InputSchema, FJsonDomBuilder::FObject& OutputSchema)
{
	FJsonDomBuilder::FObject ToolSchema;
	ToolSchema.Set(TEXT("description"), Description);
	ToolSchema.Set(TEXT("name"), Name);
	ToolSchema.Set(TEXT("inputSchema"), InputSchema);
	ToolSchema.Set(TEXT("outputSchema"), OutputSchema);
	return ToolSchema;
}

FJsonDomBuilder::FObject MakeSchema()
{
	FJsonDomBuilder::FObject Schema;
	Schema.Set(TEXT("type"), TEXT("object"));
	return Schema;
}

FJsonDomBuilder::FObject MakeProperty(const FString& Type, TSharedPtr<FJsonValue> DefaultValue)
{
	FJsonDomBuilder::FObject PropertyType;
	PropertyType.Set(TEXT("type"), Type);
	if (DefaultValue)
	{
		PropertyType.Set(TEXT("default"), DefaultValue);
	}
	return PropertyType;
}

FJsonDomBuilder::FObject MakeArrayProperty(const FString& Type)
{
	FJsonDomBuilder::FObject PropertyType;
	PropertyType.Set(TEXT("type"), TEXT("array"));
	FJsonDomBuilder::FObject ItemType;
	ItemType.Set(TEXT("type"), Type);
	PropertyType.Set(TEXT("items"), ItemType);
	return PropertyType;
}

FJsonDomBuilder::FObject MakeSetProperty(const FString& Type)
{
	FJsonDomBuilder::FObject PropertyType;
	PropertyType.Set(TEXT("type"), TEXT("array"));
	PropertyType.Set(TEXT("uniqueItems"), true);
	FJsonDomBuilder::FObject ItemType;
	ItemType.Set(TEXT("type"), Type);
	PropertyType.Set(TEXT("items"), ItemType);
	return PropertyType;
}

FJsonDomBuilder::FObject MakeMapProperty(const FString& Type)
{
	FJsonDomBuilder::FObject PropertyType;
	PropertyType.Set(TEXT("type"), TEXT("object"));
	FJsonDomBuilder::FObject ItemType;
	ItemType.Set(TEXT("type"), Type);
	PropertyType.Set(TEXT("additionalProperties"), ItemType);
	return PropertyType;
}

FJsonDomBuilder::FObject MakeNamedProperty(
	const FString& Name, const FString& Type, TSharedPtr<FJsonValue> DefaultValue)
{
	FJsonDomBuilder::FObject Property;
	FJsonDomBuilder::FObject PropertyType = MakeProperty(Type, DefaultValue);
	Property.Set(Name, PropertyType);
	return Property;
}

void AddPropertiesToSchema(FJsonDomBuilder::FObject& Schema, FJsonDomBuilder::FObject Properties)
{
	Schema.Set(TEXT("properties"), Properties);
}

void AddRequiredListToSchema(FJsonDomBuilder::FObject& Schema, const TArray<FString>& RequiredList)
{
	FJsonDomBuilder::FArray RequiredArray;
	for (FString Item : RequiredList)
	{
		RequiredArray.Add(Item);
	}
	Schema.Set(TEXT("required"), RequiredArray);
}
