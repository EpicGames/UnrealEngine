// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "JsonDomBuilder.h"

/// Utility functions for generating toolset JSON schemas.

// Call this first to create a base toolset schema object with the toolset's name, version, and description.
// Call AsJsonObject on it when you are done to generate a TSharedRef<FJsonObject> from it.
FJsonDomBuilder::FObject MakeSchemaBuilder(
	const FString& Name, const FString& Version, const FString& Description);

// Add a tool schema to the tools array and add the tools array to the schema builder.  Returns the tools
// array if you want to add more tool entries to it.
FJsonDomBuilder::FArray AddToolSchema(
	FJsonDomBuilder::FObject& Builder, FJsonDomBuilder::FArray& ToolsArray,
	FJsonDomBuilder::FObject& ToolEntry);

// Make a tool schema with the tool's name and description, input schema, and output schema.
FJsonDomBuilder::FObject MakeToolSchemaEntry(
	const FString& Name, const FString& Description,
	FJsonDomBuilder::FObject& InputSchema, FJsonDomBuilder::FObject& OutputSchema);

// Make a schema object that you can use for either an input or output schema.
FJsonDomBuilder::FObject MakeSchema();

// Make a property object for putting in an input or output schema, specifying its type and an optional
// default value.
FJsonDomBuilder::FObject MakeProperty(
	const FString& Type, TSharedPtr<FJsonValue> DefaultValue = nullptr);

// Make an array property object for putting in an input or output schema, specifying the type of its items.
FJsonDomBuilder::FObject MakeArrayProperty(const FString& Type);

// Make a set property object for putting in an input or output schema, specifying the type of its items.
FJsonDomBuilder::FObject MakeSetProperty(const FString& Type);

// Make a map property object for putting in an input or output schema, specifying the type of its items.
FJsonDomBuilder::FObject MakeMapProperty(const FString& Type);

// Make a property object for putting in an input or output schema, specifying its name, type and an optional
// default value.
FJsonDomBuilder::FObject MakeNamedProperty(
	const FString& Name, const FString& Type, TSharedPtr<FJsonValue> DefaultValue = nullptr);

// Add a property object to an input or output schema.
void AddPropertiesToSchema(FJsonDomBuilder::FObject& Schema, FJsonDomBuilder::FObject Properties);

// Add a required list of names to an input or output schema.
void AddRequiredListToSchema(FJsonDomBuilder::FObject& Schema, const TArray<FString>& RequiredList);

