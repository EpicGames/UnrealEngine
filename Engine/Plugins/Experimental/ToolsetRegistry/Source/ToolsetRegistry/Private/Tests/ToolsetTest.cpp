// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/Toolset.h"

#include "Async/Future.h"
#include "Misc/AutomationTest.h"
#include "Templates/ValueOrError.h"

#include "ToolsetRegistryTestFlags.h"
#include "FakeToolset.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::ToolsetRegistry
{
	BEGIN_DEFINE_SPEC(
		FToolsetSpec,
		"AI.ToolsetRegistry.Toolset",
		ToolsetRegistryTest::Flags)
		TSharedPtr<FFakeToolset> Toolset;
	END_DEFINE_SPEC(FToolsetSpec)

	void FToolsetSpec::Define()
	{
		Describe("IsEnabled", [this]()
		{
			BeforeEach([this]()
			{
				Toolset = MakeShared<FFakeToolset>(TEXT("Fake"));
			});

			It("can be disabled and enabled", [this]()
			{
				TestTrue(TEXT("IsEnabled_Default"), Toolset->IsEnabled());
				Toolset->SetEnabled(false);
				TestFalse(TEXT("IsEnabled_False"), Toolset->IsEnabled());
				Toolset->SetEnabled(true);
				TestTrue(TEXT("IsEnabled_True"), Toolset->IsEnabled());
			});

			It("returns empty schema when disabled", [this]()
			{
				Toolset->SetEnabled(false);
				TestTrue(TEXT("Schema_Empty"), Toolset->GetJsonSchema().IsEmpty());
			});
		});

		Describe("ToolsetLevelFilter", [this]()
		{
			BeforeEach([this]()
			{
				Toolset = MakeShared<FFakeToolset>(TEXT("Fake"));
			});

			for (const FString& Pattern : TArray<FString>{TEXT("Fake"), TEXT("/^Fake$/")})
			{
				Describe(FString::Printf(TEXT("BlockList/%s"), *Pattern), [this, Pattern]()
				{
					BeforeEach([this, Pattern]()
					{
						Toolset->SetNameFilters({Pattern}, {});
					});

					It("disables the toolset", [this]()
					{
						TestFalse(TEXT("Blocked_IsEnabled"), Toolset->IsEnabled());
					});

					It("returns empty schema", [this]()
					{
						TestTrue(TEXT("Blocked_SchemaEmpty"), Toolset->GetJsonSchema().IsEmpty());
					});

					It("returns error when tool is executed", [this]()
					{
						TValueOrError<FString, FString> Result =
							Toolset->ExecuteTool(TEXT("SomeTool"), FString()).Get();
						TestTrue(TEXT("Blocked_ExecuteError"), Result.HasError());
					});
				});

				Describe(FString::Printf(TEXT("AllowList/%s"), *Pattern), [this, Pattern]()
				{
					It("disables a toolset whose name doesn't match", [this, Pattern]()
					{
						TSharedPtr<FFakeToolset> Other = MakeShared<FFakeToolset>(TEXT("Other"));
						Other->SetNameFilters({}, {Pattern});
						TestFalse(TEXT("NonAllowed_IsDisabled"), Other->IsEnabled());
					});
				});
			}

			Describe("AllowList", [this]()
			{
				It("keeps the toolset enabled when its name matches", [this]()
				{
					Toolset->SetNameFilters({}, {TEXT("Fake")});
					TestTrue(TEXT("Allowed_IsEnabled"), Toolset->IsEnabled());
				});
			});

			Describe("BlockWinsOverAllow", [this]()
			{
				BeforeEach([this]()
				{
					Toolset->SetNameFilters({TEXT("Fake")}, {TEXT("Fake")});
				});

				It("disables the toolset when both blocked and allowed", [this]()
				{
					TestFalse(TEXT("BlockedAndAllowed_IsDisabled"), Toolset->IsEnabled());
				});
			});
		});

		Describe("PerToolFilter", [this]()
		{
			BeforeEach([this]()
			{
				Toolset = MakeShared<FFakeToolset>(TEXT("Fake"));
				Toolset->AddFakeToolCall(
					TEXT("AnotherTool"),
					MakeFulfilledPromise<TValueOrError<FString, FString>>(
						MakeValue(FFakeToolset::SUCCESSFUL_RESULT)));
			});

			Describe("BlockList", [this]()
			{
				BeforeEach([this]()
				{
					Toolset->SetNameFilters({TEXT("Fake.SomeTool")}, {});
				});

				It("excludes blocked tool from schema", [this]()
				{
					FString Schema = Toolset->GetJsonSchema();
					TestFalse(TEXT("BlockedTool_Absent"),
						Schema.Contains(TEXT("\"name\":\"Fake.SomeTool\"")));
					TestTrue(TEXT("NonBlockedTool_Present"),
						Schema.Contains(TEXT("\"name\":\"Fake.AnotherTool\"")));
					TestFalse(TEXT("BlockedTool_IsToolEnabled"),
						Toolset->IsToolEnabled(TEXT("Fake.SomeTool")));
					TestTrue(TEXT("NonBlockedTool_IsToolEnabled"),
						Toolset->IsToolEnabled(TEXT("Fake.AnotherTool")));
				});

				It("returns error when blocked tool is executed", [this]()
				{
					TValueOrError<FString, FString> Result =
						Toolset->ExecuteTool(TEXT("SomeTool"), FString()).Get();
					TestTrue(TEXT("BlockedTool_Error"), Result.HasError());
				});

				It("executes non-blocked tool successfully", [this]()
				{
					TValueOrError<FString, FString> Result =
						Toolset->ExecuteTool(TEXT("AnotherTool"), FString()).Get();
					TestTrue(TEXT("NonBlockedTool_Success"), Result.HasValue());
				});

				It("returns empty schema when all tools are blocked", [this]()
				{
					Toolset->SetNameFilters({TEXT("Fake.SomeTool"), TEXT("Fake.AnotherTool")}, {});
					TestTrue(TEXT("AllBlocked_SchemaEmpty"),
						Toolset->GetJsonSchema().IsEmpty());
				});
			});

			Describe("AllowList", [this]()
			{
				BeforeEach([this]()
				{
					Toolset->SetNameFilters({}, {TEXT("/^Fake$/"), TEXT("Fake.SomeTool")});
				});

				It("excludes non-allowed tools from schema", [this]()
				{
					FString Schema = Toolset->GetJsonSchema();
					TestTrue(TEXT("AllowedTool_Present"),
						Schema.Contains(TEXT("\"name\":\"Fake.SomeTool\"")));
					TestFalse(TEXT("NonAllowedTool_Absent"),
						Schema.Contains(TEXT("\"name\":\"Fake.AnotherTool\"")));
					TestTrue(TEXT("AllowedTool_IsToolEnabled"),
						Toolset->IsToolEnabled(TEXT("Fake.SomeTool")));
					TestFalse(TEXT("NonAllowedTool_IsToolEnabled"),
						Toolset->IsToolEnabled(TEXT("Fake.AnotherTool")));
				});

				It("executes allowed tool successfully", [this]()
				{
					TValueOrError<FString, FString> Result =
						Toolset->ExecuteTool(TEXT("SomeTool"), FString()).Get();
					TestTrue(TEXT("AllowedTool_Success"), Result.HasValue());
				});

				It("returns error when non-allowed tool is executed", [this]()
				{
					TValueOrError<FString, FString> Result =
						Toolset->ExecuteTool(TEXT("AnotherTool"), FString()).Get();
					TestTrue(TEXT("NonAllowedTool_Error"), Result.HasError());
				});
			});

			Describe("BlockWinsOverAllow", [this]()
			{
				BeforeEach([this]()
				{
					Toolset->SetNameFilters({TEXT("Fake.SomeTool")}, {TEXT("/^Fake$/"), TEXT("Fake.SomeTool")});
				});

				It("excludes tool from schema when both blocked and allowed", [this]()
				{
					TestFalse(TEXT("BlockedAndAllowed_Absent"),
						Toolset->GetJsonSchema().Contains(TEXT("\"name\":\"Fake.SomeTool\"")));
					TestFalse(TEXT("BlockedAndAllowed_IsToolEnabled"),
						Toolset->IsToolEnabled(TEXT("Fake.SomeTool")));
				});

				It("returns error when executing tool that is both blocked and allowed", [this]()
				{
					TValueOrError<FString, FString> Result =
						Toolset->ExecuteTool(TEXT("SomeTool"), FString()).Get();
					TestTrue(TEXT("BlockedAndAllowed_Error"), Result.HasError());
				});
			});

		Describe("BlockList - regex", [this]()
		{
			BeforeEach([this]()
			{
				Toolset->SetNameFilters({TEXT("/SomeTool/")}, {});
			});

			It("excludes regex-matched tool from schema", [this]()
			{
				FString Schema = Toolset->GetJsonSchema();
				TestFalse(TEXT("RegexBlocked_Absent"),
					Schema.Contains(TEXT("\"name\":\"Fake.SomeTool\"")));
				TestTrue(TEXT("RegexNonBlocked_Present"),
					Schema.Contains(TEXT("\"name\":\"Fake.AnotherTool\"")));
				TestFalse(TEXT("RegexBlocked_IsToolEnabled"),
					Toolset->IsToolEnabled(TEXT("Fake.SomeTool")));
				TestTrue(TEXT("RegexNonBlocked_IsToolEnabled"),
					Toolset->IsToolEnabled(TEXT("Fake.AnotherTool")));
			});

			It("returns error when regex-blocked tool is executed", [this]()
			{
				TValueOrError<FString, FString> Result =
					Toolset->ExecuteTool(TEXT("SomeTool"), FString()).Get();
				TestTrue(TEXT("RegexBlocked_Error"), Result.HasError());
			});

			It("executes non-matched tool successfully", [this]()
			{
				TValueOrError<FString, FString> Result =
					Toolset->ExecuteTool(TEXT("AnotherTool"), FString()).Get();
				TestTrue(TEXT("RegexNonBlocked_Success"), Result.HasValue());
			});
		});

		Describe("AllowList - regex", [this]()
		{
			BeforeEach([this]()
			{
				Toolset->SetNameFilters({}, {TEXT("/^Fake$/"), TEXT("/SomeTool/")});
			});

			It("excludes non-matched tools from schema", [this]()
			{
				FString Schema = Toolset->GetJsonSchema();
				TestTrue(TEXT("RegexAllowed_Present"),
					Schema.Contains(TEXT("\"name\":\"Fake.SomeTool\"")));
				TestFalse(TEXT("RegexNonAllowed_Absent"),
					Schema.Contains(TEXT("\"name\":\"Fake.AnotherTool\"")));
				TestTrue(TEXT("RegexAllowed_IsToolEnabled"),
					Toolset->IsToolEnabled(TEXT("Fake.SomeTool")));
				TestFalse(TEXT("RegexNonAllowed_IsToolEnabled"),
					Toolset->IsToolEnabled(TEXT("Fake.AnotherTool")));
			});

			It("executes regex-allowed tool successfully", [this]()
			{
				TValueOrError<FString, FString> Result =
					Toolset->ExecuteTool(TEXT("SomeTool"), FString()).Get();
				TestTrue(TEXT("RegexAllowed_Success"), Result.HasValue());
			});

			It("returns error when non-matched tool is executed", [this]()
			{
				TValueOrError<FString, FString> Result =
					Toolset->ExecuteTool(TEXT("AnotherTool"), FString()).Get();
				TestTrue(TEXT("RegexNonAllowed_Error"), Result.HasError());
			});
		});

		It("ListTools returns all tools regardless of filters and enabled state", [this]()
			{
				Toolset->SetNameFilters({TEXT("Fake.SomeTool")}, {});
				Toolset->SetEnabled(false);
				TArray<FString> Tools = Toolset->ListToolNames();
				Tools.Sort();
				TArray<FString> Expected = {TEXT("Fake.AnotherTool"), TEXT("Fake.SomeTool")};
				TestEqual(TEXT("ListTools_ExactMatch"), Tools, Expected);
			});
		});
	}

}  // namespace UE::ToolsetRegistry

#endif
