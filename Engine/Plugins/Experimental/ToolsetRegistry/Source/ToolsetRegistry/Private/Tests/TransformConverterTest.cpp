// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformConverterTest.h"

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "Misc/AutomationTest.h"

#include "ToolsetRegistry/ToolsetJson.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FToolsetTransformConverterSpec, "AI.ToolsetRegistry.TransformConverterSpec",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	void TestTransformSchema(const TSharedPtr<FJsonObject>& Schema);
END_DEFINE_SPEC(FToolsetTransformConverterSpec)

void FToolsetTransformConverterSpec::TestTransformSchema(const TSharedPtr<FJsonObject>& Schema)
{
	if (!TestTrue("Schema", Schema.IsValid())) return;

	FString Type;
	TestTrue("Type", Schema->TryGetStringField(TEXT("type"), Type));
	TestEqual("Type", Type, FString(TEXT("object")));

	TestFalse("No required", Schema->HasField(TEXT("required")));

	const TSharedPtr<FJsonObject>* Properties = nullptr;
	if (!(TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties)) &&
		Properties))
	{
		return;
	}
	for (const FString& Field : TArray<FString>{ TEXT("location"), TEXT("rotation"), TEXT("scale") })
	{
		const TSharedPtr<FJsonObject>* SubSchema = nullptr;
		TestTrue(*Field, Properties->Get()->TryGetObjectField(Field, SubSchema));
	}
}

void FToolsetTransformConverterSpec::Define()
{
	using namespace UE::ToolsetRegistry::Internal;
	using namespace UE::ToolsetRegistry::Internal::ToolsetJson;

	It("Converts FTransform property to transform schema", [this]()
	{
		UStruct* Struct = FTransformConverterTest::StaticStruct();
		FProperty* Prop = Struct->FindPropertyByName("TestTransform");
		const TSharedPtr<FJsonObject> Schema = PropertyToJsonSchema(Prop);
		TestTransformSchema(Schema);
	});

	It("Converts FTransform property to and from JSON", [this]()
	{
		FTransformConverterTest TestIn;
		TestIn.TestTransform = FTransform(
			FRotator(10.f, 20.f, 30.f).Quaternion(),
			FVector(100.f, 200.f, 300.f),
			FVector(2.f, 3.f, 4.f));

		UStruct* Struct = FTransformConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestTransform");

		TSharedPtr<FJsonValue> OutJson = PropertyToJsonData(Property, &TestIn.TestTransform);
		if (!TestTrue("Json", OutJson.IsValid())) return;

		TSharedPtr<FJsonObject> JsonObject = OutJson->AsObject();
		if (!TestTrue("Json object", JsonObject.IsValid())) return;

		TestTrue("location", JsonObject->HasField(TEXT("location")));
		TestTrue("rotation", JsonObject->HasField(TEXT("rotation")));
		TestTrue("scale", JsonObject->HasField(TEXT("scale")));

		FTransformConverterTest TestOut;
		TestTrue("Output", JsonDataToProperty(OutJson, Property, &TestOut.TestTransform));
		TestEqual("Location", TestOut.TestTransform.GetTranslation(),
			TestIn.TestTransform.GetTranslation());
		TestEqual("Rotation", TestOut.TestTransform.GetRotation().Rotator(),
			TestIn.TestTransform.GetRotation().Rotator());
		TestEqual("Scale", TestOut.TestTransform.GetScale3D(),
			TestIn.TestTransform.GetScale3D());
	});

	It("Partial FTransform JSON preserves unset fields", [this]()
	{
		// Verify JsonDataToProperty with a partial JSON object (only location) leaves
		// the existing rotation and scale unchanged — matching to_ue semantics.
		const FRotator OriginalRotation(45.f, 90.f, 0.f);
		const FVector OriginalScale(2.f, 2.f, 2.f);

		FTransformConverterTest Test;
		Test.TestTransform = FTransform(
			OriginalRotation.Quaternion(), FVector::ZeroVector, OriginalScale);

		UStruct* Struct = FTransformConverterTest::StaticStruct();
		FProperty* Property = Struct->FindPropertyByName("TestTransform");

		const FVector NewLocation(1.f, 2.f, 3.f);
		TSharedPtr<FJsonObject> LocationJson = MakeShared<FJsonObject>();
		LocationJson->SetNumberField(TEXT("x"), NewLocation.X);
		LocationJson->SetNumberField(TEXT("y"), NewLocation.Y);
		LocationJson->SetNumberField(TEXT("z"), NewLocation.Z);

		TSharedPtr<FJsonObject> PartialJson = MakeShared<FJsonObject>();
		PartialJson->SetObjectField(TEXT("location"), LocationJson);

		TestTrue("Output", JsonDataToProperty(
			MakeShared<FJsonValueObject>(PartialJson), Property, &Test.TestTransform));
		TestEqual("Location updated", Test.TestTransform.GetTranslation(), NewLocation);
		TestEqual("Rotation preserved", Test.TestTransform.GetRotation().Rotator(), OriginalRotation);
		TestEqual("Scale preserved", Test.TestTransform.GetScale3D(), OriginalScale);
	});
}

#endif
