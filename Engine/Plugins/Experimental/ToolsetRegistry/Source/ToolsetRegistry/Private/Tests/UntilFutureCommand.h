// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <concepts>

#include "Async/Future.h"
#include "Misc/AutomationTest.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::ToolsetRegistry::Internal
{
	// Latent test command that waits until a future is complete or a timeout elapses.
	class FUntilFutureCommand
	{
	public:
		// InFutureCompletion is called with ResultT&& when InWaitOnFuture is complete.
		template<typename ResultT> requires (!std::is_void_v<ResultT>)
		static FUntilCommand* Create(
			TFuture<ResultT>&& InWaitOnFuture,
			TFunction<void(ResultT&&)>&& InFutureCompletion,
			float InTimeout = DefaultTimeoutInSeconds)
		{
			struct FState
			{
				TFuture<ResultT> WaitOnFuture;
				TFunction<void(ResultT&&)> OnFutureCompletion;
			};
			auto State = MakeShared<FState>();
			State->WaitOnFuture = MoveTemp(InWaitOnFuture);
			State->OnFutureCompletion = MoveTemp(InFutureCompletion);
			return new FUntilCommand(
				[State]() -> bool
				{
					bool bIsReady = State->WaitOnFuture.IsReady();
					if (bIsReady)
					{
						State->OnFutureCompletion(State->WaitOnFuture.Consume());
					}
					return bIsReady;
				},
				InTimeout);
		}

		template<typename ResultT> requires std::is_void_v<ResultT>
		static FUntilCommand* Create(
			TFuture<void>&& InWaitOnFuture,
			TFunction<void()>&& InFutureCompletion,
			float InTimeout = DefaultTimeoutInSeconds)
		{
			struct FState
			{
				TFuture<void> WaitOnFuture;
				TFunction<void()> OnFutureCompletion;
			};
			auto State = MakeShared<FState>();
			State->WaitOnFuture = MoveTemp(InWaitOnFuture);
			State->OnFutureCompletion = MoveTemp(InFutureCompletion);
			return new FUntilCommand(
				[State]() -> bool
				{
					bool bIsReady = State->WaitOnFuture.IsReady();
					if (bIsReady)
					{
						State->OnFutureCompletion();
					}
					return bIsReady;
				},
				InTimeout);
		}

	public:
		static constexpr float DefaultTimeoutInSeconds = 5.0f;
	};
}

#endif  // WITH_DEV_AUTOMATION_TESTS