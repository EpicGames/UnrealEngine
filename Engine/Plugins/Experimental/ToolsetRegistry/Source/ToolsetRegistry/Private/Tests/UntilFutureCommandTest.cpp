// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"

#include "UntilFutureCommand.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(
	FUnitFutureCommandSpec,
	"AI.ToolsetRegistry.UntilFutureCommand",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FUnitFutureCommandSpec)

void FUnitFutureCommandSpec::Define()
{
	using namespace UE::ToolsetRegistry::Internal;

	Describe(TEXT("FUntilFutureCommand"), [this]()
	{
		It(TEXT("should complete with a value when a future completes"), [this]()
		{
			TSharedPtr<FString> ExpectedResult = MakeShared<FString>(TEXT("yo"));
			auto Promise = MakeShared<TPromise<FString>>();

			AddCommand(
				new FDelayedFunctionLatentCommand(
					[Promise, ExpectedResult]() -> void
					{
						Promise->SetValue(*ExpectedResult);
					},
					0.1f /* wait 100ms */));
			AddCommand(
				FUntilFutureCommand::Create<FString>(
					Promise->GetFuture(),
					[this, ExpectedResult](FString&& Result) -> void
					{
						TestEqual(TEXT("result"), Result, *ExpectedResult);
					}));
		});

		It(TEXT("should complete with no value when a future completes"), [this]()
		{
			auto Promise = MakeShared<TPromise<void>>();
			auto Completed = MakeShared<bool>(false);

			AddCommand(
				new FDelayedFunctionLatentCommand(
					[Promise]() -> void
					{
						Promise->EmplaceValue();
					},
					0.1f /* wait 100ms */));
			AddCommand(
				FUntilFutureCommand::Create<void>(
					Promise->GetFuture(),
					[this, Completed]() -> void
					{
						*Completed = true;
					}));
			AddCommand(
				new FFunctionLatentCommand(
					[this, Completed]() -> bool
					{
						TestTrue(TEXT("Completed"), *Completed);
						return true;
					}));
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS