// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Templates/ValueOrError.h"

#include "ToolsetRegistry/ValueOrErrorFuture.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(
	FToolsetRegistryValueOrErrorFutureSpec,
	"AI.ToolsetRegistry.ValueOrErrorFuture",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FToolsetRegistryValueOrErrorFutureSpec)

void FToolsetRegistryValueOrErrorFutureSpec::Define()
{
	using namespace UE::ToolsetRegistry::Internal;

	Describe(TEXT("Make"), [this]()
	{
		It(TEXT("should return a value"), [this]()
		{
			FString ExpectedValue = TEXT("Yo");
			TFuture<TValueOrError<FString, FString>> Future =
				FStringValueOrErrorFuture::Make(MakeValue(ExpectedValue));
			if (!TestTrue(TEXT("is ready"), Future.IsReady())) return;

			auto ValueOrValue = Future.Get();
			if (!TestTrue(TEXT("has value"), ValueOrValue.HasValue())) return;

			TestEqual(TEXT("value"), ValueOrValue.GetValue(), ExpectedValue);
		});

		It(TEXT("should return an error"), [this]()
		{
			FString ExpectedError = TEXT("Failed");
			TFuture<TValueOrError<FString, FString>> Future =
				FStringValueOrErrorFuture::Make(MakeError(FString(ExpectedError)));
			if (!TestTrue(TEXT("is ready"), Future.IsReady())) return;

			auto ValueOrError = Future.Get();
			if (!TestTrue(TEXT("has error"), ValueOrError.HasError())) return;

			TestEqual(TEXT("error"), ValueOrError.GetError(), ExpectedError);
		});
	});

	Describe(TEXT("MakeError"), [this]()
	{
		It(TEXT("should return an error"), [this]()
		{
			FString ExpectedError = TEXT("Failed");
			TFuture<TValueOrError<FString, FString>> Future =
				FStringValueOrErrorFuture::MakeError(FString(ExpectedError));
			if (!TestTrue(TEXT("is ready"), Future.IsReady())) return;

			auto ValueOrError = Future.Get();
			if (!TestTrue(TEXT("has error"), ValueOrError.HasError())) return;

			TestEqual(TEXT("error"), ValueOrError.GetError(), ExpectedError);
		});
	});

	Describe(TEXT("MakeValue"), [this]()
	{
		It(TEXT("should return a value"), [this]()
		{
			FString ExpectedValue = TEXT("Yo");
			TFuture<TValueOrError<FString, FString>> Future =
				FStringValueOrErrorFuture::MakeValue(FString(ExpectedValue));
			if (!TestTrue(TEXT("is ready"), Future.IsReady())) return;

			auto ValueOrValue = Future.Get();
			if (!TestTrue(TEXT("has value"), ValueOrValue.HasValue())) return;

			TestEqual(TEXT("value"), ValueOrValue.GetValue(), ExpectedValue);
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
