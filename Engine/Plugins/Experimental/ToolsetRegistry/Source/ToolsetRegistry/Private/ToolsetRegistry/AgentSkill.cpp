// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/AgentSkill.h"

#include "AssetToolsModule.h"
#include "EditorAssetLibrary.h"
#include "Factories/BlueprintFactory.h"
#include "Interfaces/ITargetPlatform.h"
#include "Internationalization/Regex.h"
#include "Kismet/KismetSystemLibrary.h"

#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/NamePatternFilter.h"
#include "ToolsetRegistry/ToolsetLibrary.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AgentSkill)

FAgentSkillDetails UAgentSkill::GetDetails() const
{
	FAgentSkillDetails SkillDetails;
	SkillDetails.Instructions = GeneratePrompt(Instructions);
	return SkillDetails;
}

TMap<FString, FString> UAgentSkillToolset::ListSkills()
{
	const UToolsetRegistrySettings* Settings = GetDefault<UToolsetRegistrySettings>();
	const TArray<FRegexPattern> BlockPatterns =
		UE::ToolsetRegistry::Internal::CompilePatterns(Settings->AgentSkillBlockedNames);
	const TArray<FRegexPattern> AllowPatterns =
		UE::ToolsetRegistry::Internal::CompilePatterns(Settings->AgentSkillAllowedNames);

	TMap<FString, FString> Summaries;
	TArray<FSoftClassPath> DerivedClasses = UToolsetLibrary::GetDerivedClasses(UAgentSkill::StaticClass());
	for (const auto& DerivedClass : DerivedClasses)
	{
		UClass* AgentSkillClass = DerivedClass.TryLoadClass<UAgentSkill>();
		if (!AgentSkillClass)
		{
			UE_LOGF(LogToolsetRegistry, Warning,
				"Unable to load agent skill: %ls", *DerivedClass.ToString());
			continue;
		}
		const FString SkillPath = AgentSkillClass->GetPathName();
		if (!UE::ToolsetRegistry::Internal::IsNameAllowed(SkillPath, BlockPatterns, AllowPatterns))
		{
			continue;
		}
		UAgentSkill* AgentSkill = Cast<UAgentSkill>(AgentSkillClass->GetDefaultObject());
		if (AgentSkill && !AgentSkill->Description.IsEmpty())
		{
			Summaries.Add(SkillPath, AgentSkill->Description);
		}
	}

	return Summaries;
}

TMap<FString, FAgentSkillDetails> UAgentSkillToolset::GetSkills(const TArray<FString>& SkillPaths)
{
	const UToolsetRegistrySettings* Settings = GetDefault<UToolsetRegistrySettings>();
	const TArray<FRegexPattern> BlockPatterns =
		UE::ToolsetRegistry::Internal::CompilePatterns(Settings->AgentSkillBlockedNames);
	const TArray<FRegexPattern> AllowPatterns =
		UE::ToolsetRegistry::Internal::CompilePatterns(Settings->AgentSkillAllowedNames);

	TMap<FString, FAgentSkillDetails> Skills;
	for (const auto& SkillPath : SkillPaths)
	{
		if (!UE::ToolsetRegistry::Internal::IsNameAllowed(SkillPath, BlockPatterns, AllowPatterns))
		{
			continue;
		}
		UClass* AgentSkillClass = LoadObject<UClass>(nullptr, SkillPath);
		if (AgentSkillClass)
		{
			UAgentSkill* AgentSkill = Cast<UAgentSkill>(AgentSkillClass->GetDefaultObject());
			if (AgentSkill)
			{
				Skills.Add(SkillPath, AgentSkill->GetDetails());
			}
		}
	}
	return Skills;
}

FString UAgentSkillToolset::CreateSkill(
	const FString& FolderPath, const FString& AssetName, const FString& Description,
	const FAgentSkillDetails& Details)
{
	if (AssetName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString(TEXT("Cannot create a skill with no name.")));
		return FString();
	}

	if (FolderPath.IsEmpty() || FolderPath[0] != '/')
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("%s is not a valid folder path."), *FolderPath));
		return FString();
	}
	
	UBlueprintFactory* BlueprintFactory = NewObject<UBlueprintFactory>();
	BlueprintFactory->ParentClass = UAgentSkill::StaticClass();

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	UBlueprint* SkillBlueprint = Cast<UBlueprint>(AssetTools.CreateAsset(
		AssetName, FolderPath, UBlueprint::StaticClass(), BlueprintFactory));

	if (!SkillBlueprint)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Unable to create AgentSkill Blueprint %s in %s."),
				*AssetName, *FolderPath));
		return FString();
	}

	UClass* AgentSkillClass = SkillBlueprint->GeneratedClass;
	FString AgentClassPath = AgentSkillClass->GetPathName();
	if (!UpdateSkill(AgentClassPath, Description, Details))
	{
		UEditorAssetLibrary::DeleteLoadedAssets({ SkillBlueprint });
		// No need for an error as it will be populated by UpdateSkill.
		return FString();
	}

	return AgentClassPath;
}

bool UAgentSkillToolset::UpdateSkill(
	const FString& SkillPath, const FString& Description, const FAgentSkillDetails& Details)
{
	UClass* AgentSkillClass = LoadObject<UClass>(nullptr, SkillPath);
	if (!AgentSkillClass)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("%s is not an AgentSkill asset."), *SkillPath));
		return false;
	}

	UAgentSkill* AgentSkill = Cast<UAgentSkill>(AgentSkillClass->GetDefaultObject());
	if (!AgentSkill)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("%s is not an AgentSkill asset."), *SkillPath));
		return false;
	}

	// Native C++ classes and transient classes (e.g. Python-generated UClasses) have no
	// saveable asset backing. Mutating the CDO would appear to succeed but the change would
	// be lost the next time the source module is reloaded or the editor restarts. Only
	// Blueprint-asset-backed skills can be edited through this tool.
	if (AgentSkillClass->HasAnyClassFlags(CLASS_Native)
		|| AgentSkillClass->HasAnyFlags(RF_Transient))
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("%s cannot be updated through this tool because it is not backed by a saveable asset. Edit the source script or code that defines it."), *SkillPath));
		return false;
	}

	AgentSkill->Description = Description;
	AgentSkill->Instructions = Details.Instructions;
	AgentSkill->Modify();
	AgentSkillClass->GetPackage()->MarkPackageDirty();
	return true;
}
