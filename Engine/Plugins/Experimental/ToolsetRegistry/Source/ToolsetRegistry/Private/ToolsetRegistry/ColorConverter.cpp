// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorConverter.h"

#include "JsonObjectConverter.h"
#include "JsonSchema/JsonSchemaGenerator.h"
#include "Kismet/KismetSystemLibrary.h"

#include "ToolsetRegistry/JsonConversion.h"

namespace
{
	const FStructProperty* AsColorProperty(TNotNull<const FProperty*> Property)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			const FName StructName = StructProperty->Struct->GetFName();
			if (StructName == NAME_Color || StructName == NAME_LinearColor)
			{
				return StructProperty;
			}
		}
		return nullptr;
	}

	bool IsLinear(const FProperty* Property)
	{
		const FStructProperty* ColorProperty = AsColorProperty(Property);
		check(ColorProperty);
		return ColorProperty->Struct->GetFName() == NAME_LinearColor;
	}
}

namespace UE::ToolsetRegistry
{
	FString FToolsetColorConverter::GetName() const
	{
		static FString Name(TEXT("ColorConverter"));
		return Name;
	}

	bool FToolsetColorConverter::CanConvertProperty(TNotNull<const FProperty*> Property)
	{
		return AsColorProperty(Property) != nullptr;
	}

	TSharedPtr<FJsonObject> FToolsetColorConverter::PropertyToJsonSchema(
		TNotNull<const FProperty*> Property)
	{
		bool IsLinearProperty = IsLinear(Property);
		TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::UStructToJsonSchemaObject(
			TBaseStructure<FLinearColor>::Get());
		TSharedPtr<FJsonObject> Properties = Schema->GetObjectField(TEXT("properties"));
		TArray<FString> PropertyNames({ "r", "g", "b", "a" });
		for (const auto& Name : PropertyNames)
		{
			TSharedPtr<FJsonObject> Component = Properties->GetObjectField(Name);
			Component->SetNumberField(FString(TEXT("minimum")), 0.f);
			if (!IsLinearProperty)
			{
				Component->SetNumberField(FString(TEXT("maximum")), 1.f);
			}
		}
		// The default description isn't very useful and includes a source code reference,
		// so we just remove it to save some tokens.
		Schema->RemoveField(FString(TEXT("description")));
		return Schema;
	}

	TSharedPtr<FJsonValue> FToolsetColorConverter::PropertyToDefault(
		TNotNull<const FProperty*> Property, const FString& DefaultString)
	{
		FLinearColor DefaultLinearColor = FLinearColor::Black;
		if (IsLinear(Property))
		{
			Property->ImportText_Direct(*DefaultString, &DefaultLinearColor, nullptr, PPF_None);
		}
		else
		{
			FColor DefaultColor = FColor::Black;
			Property->ImportText_Direct(*DefaultString, &DefaultColor, nullptr, PPF_None);
			DefaultLinearColor = DefaultColor.ReinterpretAsLinear();
		}
		TSharedPtr<FJsonObject> DefaultObject = MakeShared<FJsonObject>();
		FJsonObjectConverter::UStructToJsonObject(
			TBaseStructure<FLinearColor>::Get(), &DefaultLinearColor, DefaultObject.ToSharedRef());
		return MakeShared<FJsonValueObject>(DefaultObject);
	}

	TSharedPtr<FJsonValue> FToolsetColorConverter::PropertyToJsonData(
		TNotNull<FProperty*> Property, const void* Value)
	{
		FLinearColor OutColor;
		if (IsLinear(Property))
		{
			OutColor = *(static_cast<const FLinearColor*>(Value));
		}
		else
		{
			const FColor& InColor = *(static_cast<const FColor*>(Value));
			OutColor = InColor.ReinterpretAsLinear();
		}
		TSharedPtr<FJsonObject> OutJson = MakeShared<FJsonObject>();
		FJsonObjectConverter::UStructToJsonObject(
			TBaseStructure<FLinearColor>::Get(), &OutColor, OutJson.ToSharedRef());
		return MakeShared<FJsonValueObject>(OutJson);
	}

	bool FToolsetColorConverter::JsonDataToProperty(
		const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
		void* OutValue, UObject* Outer)
	{
		TSharedPtr<FJsonObject> JsonObject = JsonValue->AsObject();
		
		FLinearColor LinearColor;
		if (FJsonObjectConverter::JsonObjectToUStruct(
			JsonObject.ToSharedRef(), TBaseStructure<FLinearColor>::Get(), &LinearColor))
		{
			if (IsLinear(Property))
			{
				FLinearColor& LinearColorOut = *(static_cast<FLinearColor*>(OutValue));
				LinearColorOut = LinearColor;
			}
			else
			{
				FColor& ColorOut = *(static_cast<FColor*>(OutValue));
				ColorOut = LinearColor.ToFColor(false);
			}
		}
		else
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("%s is not valid JSON for Property %s"),
				*Internal::JsonToString(JsonValue.ToSharedRef()), *Property->GetName()));
		}
		return true;
	}
}