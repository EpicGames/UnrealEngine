// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetJsonConverter.h"

namespace UE::ToolsetRegistry
{
	/// Converts FColor and FLinearColor.
	class FToolsetColorConverter : public FToolsetJsonConverter
	{
	public:
		virtual FString GetName() const override;

		virtual bool CanConvertProperty(TNotNull<const FProperty*> Property)  override;

		virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(
			TNotNull<const FProperty*> Property)  override;

		virtual TSharedPtr<FJsonValue> PropertyToDefault(
			TNotNull<const FProperty*> Property, const FString& DefaultString)  override;

		virtual TSharedPtr<FJsonValue> PropertyToJsonData(
			TNotNull<FProperty*> Property, const void* Value)  override;

		virtual bool JsonDataToProperty(
			const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
			void* OutValue, UObject* Outer)  override;
	};
}

