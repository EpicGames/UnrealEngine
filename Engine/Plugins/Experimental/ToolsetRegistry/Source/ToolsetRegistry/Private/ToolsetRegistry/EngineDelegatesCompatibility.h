// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

// Temporarily used to determine whether FCoreDelegates::GetOnPostEngineInit() is
// available until it's present in all streams.
template<typename T>
struct FOnPostEngineInit
{
	static FSimpleMulticastDelegate& Get()
	{
		if constexpr (requires { T::GetOnPostEngineInit(); })
		{
			return T::GetOnPostEngineInit();
		}
		else
		{
			return T::OnPostEngineInit;
		}
	}
};