// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/FunctionLibraryToolset.h"

#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "UObject/FieldIterator.h"

#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/ObjectFunctionToolCall.h"
#include "ToolsetRegistry/ToolsetDefinition.h"
#include "ToolsetRegistry/ToolsetJson.h"
#include "ToolsetRegistry/ValueOrErrorFuture.h"

namespace UE::ToolsetRegistry
{

	FFunctionLibraryToolset::FFunctionLibraryToolset(TSubclassOf<UToolsetDefinition> LibraryClass) :
		ToolsetClass(LibraryClass)
	{
		GenerateToolCallObjects();
		ToolsetName = ToolsetClass.Pin()
			? FFunctionLibraryToolset::GetToolsetClassName(LibraryClass) : FString();
	}

	FString FFunctionLibraryToolset::GetJsonSchemaInternal() const
	{
		TSharedPtr<FJsonObject> Schema = GenerateSchema();
		return UE::ToolsetRegistry::Internal::JsonToString(Schema.ToSharedRef());
	}

	FString FFunctionLibraryToolset::GetToolsetVersion() const
	{
		static const FString UnknownVersion(TEXT("Unknown"));
		TStrongObjectPtr<UClass> Toolset = ToolsetClass.Pin();
		if (!Toolset)
		{
			return UnknownVersion;
		}

		const UToolsetDefinition* ToolsetInterface = Cast<UToolsetDefinition>(Toolset.Get()->GetDefaultObject());
		if (!ensure(ToolsetInterface))
		{
			return UnknownVersion;
		}

		return ToolsetInterface->GetToolsetVersion();
	}

	TSharedPtr<FJsonObject> FFunctionLibraryToolset::GenerateSchema() const
	{
		TSharedPtr<FJsonObject> Schema = MakeShared<FJsonObject>();
		check(Schema);
		Schema->SetStringField(FString(TEXT("name")), GetToolsetName());
		Schema->SetStringField(FString(TEXT("version")), GetToolsetVersion());
		Schema->SetStringField(FString(TEXT("description")), GetToolsetDescription());

		TArray<TSharedPtr<FJsonValue>> ToolsArray;
		for (TMap<FString, TSharedPtr<FObjectFunctionToolCall>>::TConstIterator
			ToolsIter = Tools.CreateConstIterator(); ToolsIter; ++ToolsIter)
		{
			if (const FObjectFunctionToolCall* const ToolCall = ToolsIter.Value().Get())
			{
				const UFunction* const Function = ToolCall->GetFunction();
				if (!Function)
				{
					UE_LOG(LogToolsetRegistry, Verbose,
						TEXT("Skipping schema for '%s' in toolset '%s': UFunction no longer valid."),
						*ToolsIter.Key(), *ToolsetName);
					continue;
				}

				const TSharedPtr<FJsonObject> ToolEntry =
					UE::ToolsetRegistry::Internal::ToolsetJson::StructToJsonSchema(Function);

				// Modify tool name from "ClassName.FunctionName" to "ToolsetName.FunctionName".
				const FString ToolName = ToolEntry->GetStringField(TEXT("name"));
				int32 LastDotIndex;
				// Assert if there is no dot, that would mean schema generation has failed catastrophically.
				if (ensure(ToolName.FindLastChar('.', LastDotIndex)))
				{
					const FString FunctionName = ToolName.RightChop(LastDotIndex + 1);
					const FString NewToolName = GetToolsetName() + TEXT(".") + FunctionName;
					ToolEntry->SetStringField(TEXT("name"), *NewToolName);
				}

				ToolsArray.Add(MakeShared<FJsonValueObject>(ToolEntry));
			}
		}
		Schema->SetArrayField(FString(TEXT("tools")), ToolsArray);
		return Schema;
	}

	void FFunctionLibraryToolset::GenerateToolCallObjects()
	{
		if (!Tools.IsEmpty())
		{
			Tools.Empty();
		}

		TStrongObjectPtr<UClass> Toolset = ToolsetClass.Pin();
		if (!Toolset)
		{
			return;
		}

		for (TFieldIterator<UFunction> FuncIt(Toolset.Get(), EFieldIterationFlags::None); FuncIt; ++FuncIt)
		{
			TObjectPtr<UFunction> Function = *FuncIt;
			if (Function)
			{
				const TValueOrError<bool, FString> IsCallableOrError = UToolsetDefinition::IsFunctionAICallable(Function);
				if (IsCallableOrError.HasValue())
				{
					if (IsCallableOrError.GetValue())
					{
						Tools.Add(
							Function->GetName(),
							FObjectFunctionToolCall::Create(
								TNotNull<UObject*>(Toolset.Get()), TNotNull<UFunction*>(Function)));
					}
				}
				else
				{
					UE_LOG(LogToolsetRegistry, Error, TEXT("UFunction '%ls' %ls and will not be added as a Tool."),
						*Function->GetPathName(), IsCallableOrError.HasError() ? *IsCallableOrError.GetError() : TEXT("has an unknown error"));
				}
			}
		}
	}

	TFuture<TValueOrError<FString, FString>> FFunctionLibraryToolset::ExecuteToolInternal(
		const FString& ToolName, const FString& JsonInput)
	{
		using namespace UE::ToolsetRegistry::Internal;

		const TStrongObjectPtr<UClass> Toolset = ToolsetClass.Pin();
		if (!Toolset)
		{
			return FStringValueOrErrorFuture::MakeError(TEXT("Invalid Toolset Class"));
		}

		const TSharedPtr<FObjectFunctionToolCall>* ToolCallPtr = Tools.Find(ToolName);
		if (!ToolCallPtr)
		{
			return FStringValueOrErrorFuture::MakeError(
				FString::Printf(TEXT("Unknown tool %s"), *ToolName));
		}

		TSharedPtr<FObjectFunctionToolCall> ToolCall = *ToolCallPtr;
		check(ToolCall);

		FObjectFunctionToolCall::FFunctionInputParamsJson JsonParams;
		JsonParams.Set<FString>(JsonInput);

		// Capture script errors raised by the tool during execution. Note: script errors raised by
		// tools running outside of the game thread will not be captured correctly.
		TSharedPtr<FToolCallExceptionHandler> ExceptionHandler = MakeShared<FToolCallExceptionHandler>();
		ensureAlways(IsInGameThread());

		const FString ToolNameCopy = ToolName;
		return ToolCall->Execute(JsonParams, ExceptionHandler).Next(
			[
				Toolset,  // Capture to keep alive.
				ToolCall = ToolCall,
				ExceptionHandler = ExceptionHandler,
				ToolNameCopy = ToolNameCopy
			](FJsonValueOrError&& ToolCallResult) -> TValueOrError<FString, FString>
			{
				ensureAlways(IsInGameThread());
				if (ToolCallResult.HasError())
				{
					return MakeError(ToolCallResult.StealError());
				}
				else if (
					FString ScriptException = ExceptionHandler->GetException();
					!ScriptException.IsEmpty())
				{
					// If the tool raised an error, return that.
					return MakeError(MoveTemp(ScriptException));
				}
				check(ToolCallResult.HasValue());
				TSharedPtr<FJsonValue> ResultValue = ToolCallResult.StealValue();
				if (!ResultValue.IsValid())
				{
					return MakeError(
						FString::Printf(
							TEXT("Unable to serialize result from executing tool %s"),
							*ToolNameCopy));
				}
				return MakeValue(
					UE::ToolsetRegistry::Internal::JsonToString(ResultValue.ToSharedRef()));
			});

	}

	TArray<FString> FFunctionLibraryToolset::GetToolNames() const
	{
		TArray<FString> ToolNames;
		Tools.GenerateKeyArray(ToolNames);
		return ToolNames;
	}

	bool FFunctionLibraryToolset::HasValidTools() const
	{
		return !Tools.IsEmpty();
	}

	/*static*/ FString FFunctionLibraryToolset::GetToolsetClassName(const UClass* InClass)
	{
		FString Name;
		if (InClass == nullptr)
		{
			return Name;
		}
		const FString ClassName = InClass->GetName();
		Name = ClassName;
		check(InClass->GetOuter());
		const FString PackageName = InClass->GetOuter()->GetName();
		int32 LastSlashIndex;
		FString QualifierName;
		if (ensure(PackageName.FindLastChar('/', LastSlashIndex)))
		{
			// Convert package name like "/Script/ToolsetRegistry" to a shorter qualifier like
			// "ToolsetRegistry".
			QualifierName = PackageName.RightChop(LastSlashIndex + 1);
			if (QualifierName.EndsWith(FString("_PY")))
			{
				// Special handling for Python packages: convert package name like
				// "/ToolsetRegistry/Python/toolset_registry/tests/demo_toolset_PY" as generated by
				// PyUtil::GetGeneratedTypeOuterAndName, to "toolset_registry.tests.demo_toolset".
				TArray<FString> PackageParts;
				PackageName.ParseIntoArray(PackageParts, TEXT("/"));
				int32 PythonPartIndex;
				if (PackageParts.FindLast(TEXT("Python"), PythonPartIndex))
				{
					PackageParts.RemoveAt(0, PythonPartIndex + 1);
				}
				if (int32 ChopIndex = PackageParts.Last().Find(TEXT("_PY")); ChopIndex != INDEX_NONE)
				{
					PackageParts.Last() = PackageParts.Last().Left(ChopIndex);
				}
				QualifierName = FString::Join(PackageParts, TEXT("."));
			}

			Name = QualifierName + TEXT(".") + ClassName;
		}
		return Name;
	}
}
