// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/Map.h"
#include "Dom/JsonObject.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/ValueOrError.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "ToolsetRegistry/Toolset.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

namespace UE::ToolsetRegistry
{
	class FObjectFunctionToolCall;

	// Wraps a UToolsetDefinition for use as a Toolset.
	class FFunctionLibraryToolset : public FToolset
	{
	public:

		explicit FFunctionLibraryToolset(TSubclassOf<UToolsetDefinition> LibraryClass);

		//~ Begin FToolset API
		// Returns the name of the toolset
		virtual FString GetToolsetName() const override
		{
			return ToolsetName;
		}

		// Returns the version of the toolset
		virtual FString GetToolsetVersion() const override;

		// Returns the UClass that backs this toolset.
		virtual UClass* GetToolsetClass() const override { return ToolsetClass.Get(); }
		//~End FToolset API

		// Returns the description of the toolset from the class's metadata.
		virtual FString GetToolsetDescription() const override
		{
			const TStrongObjectPtr<UClass> Pinned = ToolsetClass.Pin();
			if (Pinned)
			{
				return Pinned->GetToolTipText().ToString();
			}
			return FString();
		}

		TArray<FString> GetToolNames() const;

		// Whether the toolset has a valid set of tools.
		bool HasValidTools() const;

		// Get qualified name for toolset class.
		static FString GetToolsetClassName(const UClass* InClass);

	protected:
		// Executes a tool with the given name and JSON input, returning a future with the JSON
		// output.
		virtual TFuture<TValueOrError<FString, FString>> ExecuteToolInternal(
			const FString& ToolName, const FString& JsonInput) override;

		// Returns the JSON schema for the toolset
		virtual FString GetJsonSchemaInternal() const override;

	private:
		// The wrapped toolset function library
		TWeakObjectPtr<UClass> ToolsetClass;

		// Cached toolset name to allow unregistering even when the wrapped toolset class is no longer
		// available.
		FString ToolsetName;

		// Map of valid tool names to tool call objects.
		TMap<FString, TSharedPtr<FObjectFunctionToolCall>> Tools;

		// Generate schema from reflection data.
		TSharedPtr<FJsonObject> GenerateSchema() const;

		// Generate Tools list.
		void GenerateToolCallObjects();
	};
}
