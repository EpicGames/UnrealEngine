// Copyright Epic Games, Inc. All Rights Reserved.

#include "GuidConverter.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/Guid.h"
#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/Module.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

namespace UE::ToolsetRegistry
{
	namespace
	{
		bool IsGuidProperty(TNotNull<const FProperty*> Property)
		{
			const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
			return StructProperty && StructProperty->Struct == TBaseStructure<FGuid>::Get();
		}
	}

	FString FToolsetGuidConverter::GetName() const
	{
		static FString Name(TEXT("GuidConverter"));
		return Name;
	}

	bool FToolsetGuidConverter::CanConvertProperty(TNotNull<const FProperty*> Property)
	{
		return IsGuidProperty(Property);
	}

	TSharedPtr<FJsonObject> FToolsetGuidConverter::PropertyToJsonSchema(TNotNull<const FProperty*> Property)
	{
		TSharedPtr<FJsonObject> Schema = MakeShared<FJsonObject>();
		Schema->SetStringField(TEXT("type"), TEXT("string"));
		Schema->SetStringField(TEXT("description"),
			TEXT("Globally unique identifier in 8-4-4-4-12 hyphenated form, e.g. \"E05FCC13-4D37-9D7A-E238-83859F29AD74\"."));
		Schema->SetStringField(TEXT("pattern"), TEXT("^[0-9A-Fa-f]{8}-[0-9A-Fa-f]{4}-[0-9A-Fa-f]{4}-[0-9A-Fa-f]{4}-[0-9A-Fa-f]{12}$"));
		return Schema;
	}

	TSharedPtr<FJsonValue> FToolsetGuidConverter::PropertyToDefault(TNotNull<const FProperty*> Property, const FString& DefaultString)
	{
		FGuid DefaultGuid;
		if (!DefaultString.IsEmpty())
		{
			if (Property->ImportText_Direct(*DefaultString, &DefaultGuid, nullptr, PPF_None) == nullptr)
			{
				UE_LOGF(LogToolsetRegistry, Warning,
					"Failed to parse default GUID '%ls' for property '%ls'; using zero GUID.",
					*DefaultString, *Property->GetName());
			}
		}
		return MakeShared<FJsonValueString>(DefaultGuid.ToString(EGuidFormats::DigitsWithHyphens));
	}

	TSharedPtr<FJsonValue> FToolsetGuidConverter::PropertyToJsonData(TNotNull<FProperty*> Property, const void* Value)
	{
		const FGuid& Guid = *static_cast<const FGuid*>(Value);
		return MakeShared<FJsonValueString>(Guid.ToString(EGuidFormats::DigitsWithHyphens));
	}

	bool FToolsetGuidConverter::JsonDataToProperty(const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, void* OutValue, UObject* Outer)
	{
		if (!JsonValue.IsValid())
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("null JSON value for property '%s'"), *Property->GetName()));
			return true;
		}

		if (JsonValue->Type == EJson::String)
		{
			FGuid ParsedGuid;
			if (FGuid::ParseExact(JsonValue->AsString(), EGuidFormats::DigitsWithHyphens, ParsedGuid))
			{
				*static_cast<FGuid*>(OutValue) = ParsedGuid;
				return true;
			}
		}

		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("'%s' is not a valid 8-4-4-4-12 hyphenated GUID for property '%s'"),
			*Internal::JsonToString(JsonValue.ToSharedRef()), *Property->GetName()));
		// We return true to prevent FJsonObjectConverter from falling through to its permissive
		// native FGuid handler, which would silently overwrite our rejection.
		return true;
	}
}
