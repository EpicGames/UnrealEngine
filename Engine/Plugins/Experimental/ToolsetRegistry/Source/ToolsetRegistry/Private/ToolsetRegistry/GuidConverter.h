// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetJsonConverter.h"

namespace UE::ToolsetRegistry
{
	/**
	 * Converts FGuid to/from a JSON string in FGuid::EGuidFormats::DigitsWithHyphens form.
	 *
	 * Quality-of-life converter for agents. Without it, the default FJsonObjectConverter
	 * exposes FGuid as an object {a, b, c, d} of four uint32s — agents have to manually
	 * plumb the four parts around and sometimes echo them back to the user, which is
	 * unreadable. With the converter the schema is a single hex string in standard
	 * 8-4-4-4-12 form, so the reflected GUID is human-recognizable in transcripts.
	 */
	class FToolsetGuidConverter : public FToolsetJsonConverter
	{
	public:
		virtual FString GetName() const override;

		virtual bool CanConvertProperty(TNotNull<const FProperty*> Property) override;

		virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(
			TNotNull<const FProperty*> Property) override;

		virtual TSharedPtr<FJsonValue> PropertyToDefault(
			TNotNull<const FProperty*> Property, const FString& DefaultString) override;

		virtual TSharedPtr<FJsonValue> PropertyToJsonData(
			TNotNull<FProperty*> Property, const void* Value) override;

		virtual bool JsonDataToProperty(
			const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
			void* OutValue, UObject* Outer) override;
	};
}
