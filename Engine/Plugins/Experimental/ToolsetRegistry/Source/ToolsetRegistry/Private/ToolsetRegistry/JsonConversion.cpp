// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/JsonConversion.h"

#include "Misc/AssertionMacros.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Templates/Function.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

namespace UE::ToolsetRegistry::Internal
{
	TSharedPtr<FJsonObject> JsonStringToJsonObject(const FString& JsonString)
	{
		TSharedPtr<FJsonObject> ReturnJsonObject;
		if (!JsonString.IsEmpty())
		{
			const TSharedRef<TJsonReader<>> JsonReader =
				TJsonReaderFactory<>::Create(JsonString);
			TSharedPtr<FJsonObject> JsonObject;
			if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
			{
				ReturnJsonObject = JsonObject;
			}
		}
		return ReturnJsonObject;
	}

	TSharedRef<FJsonObject> JsonObjectOrEmpty(TSharedPtr<FJsonObject> JsonObject)
	{
		return JsonObject ? JsonObject.ToSharedRef() : MakeShared<FJsonObject>();
	}

	TSharedPtr<FJsonValue> JsonStringToJsonValue(const FString& JsonString)
	{
		// Work around JSON deserialization's lack of support for loading values from strings
		// by wrapping in a temporary object.
		TSharedPtr<FJsonValue> ReturnJsonValue;
		if (!JsonString.IsEmpty())
		{
			static const FString Key = TEXT("__key");
			TSharedPtr<FJsonObject> JsonObject =
				JsonStringToJsonObject(
					FString::Printf(TEXT(R"json({"%s": %s})json"), *Key, *JsonString));
			if (JsonObject)
			{
				ReturnJsonValue = JsonObject->Values.FindAndRemoveChecked(*Key);
			}
		}
		return ReturnJsonValue;
	}

	namespace
	{
		using FJsonPrintPolicy = TCondensedJsonPrintPolicy<TCHAR>;
		using FJsonWriterFactory = TJsonWriterFactory<FJsonPrintPolicy::CharType, FJsonPrintPolicy>;
		using FJsonWriter = TJsonWriter<FJsonPrintPolicy::CharType, FJsonPrintPolicy>;

		FString WithJsonWriter(TFunction<void(TSharedRef<FJsonWriter>)> Apply)
		{
			FString JsonString;
			TSharedRef<FJsonWriter> JsonWriter = FJsonWriterFactory::Create(&JsonString);
			Apply(JsonWriter);
			return JsonString;
		}
	}

	FString JsonToString(const TSharedRef<FJsonObject> JsonObject)
	{
		return WithJsonWriter(
			[JsonObject](auto JsonWriter) -> void
			{
				FJsonSerializer::Serialize(JsonObject, JsonWriter);
			});
	}

	FString JsonToString(const TSharedRef<FJsonValue> JsonValue)
	{
		FString JsonString = WithJsonWriter(
			[JsonValue](auto JsonWriter) -> void
			{
				FJsonSerializer::Serialize(JsonValue, TEXT(""), JsonWriter);
			});
		return JsonString;
	}
}