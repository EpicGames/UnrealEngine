// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Templates/SharedPointer.h"

namespace UE::ToolsetRegistry::Internal
{
	// Convert a JSON string to a JSON object or return a null pointer if the string cannot be
	// parsed or isn't an object.
	TSharedPtr<FJsonObject> JsonStringToJsonObject(const FString& JsonString);

	// Return either the provided JSON object or return an empty object.
	TSharedRef<FJsonObject> JsonObjectOrEmpty(TSharedPtr<FJsonObject> JsonObject);

	// Convert a JSON string to a JSON value or return a null pointer if the string cannot be
	// parsed.
	TSharedPtr<FJsonValue> JsonStringToJsonValue(const FString& JsonString);

	// Convert a JSON object to a string.
	FString JsonToString(const TSharedRef<FJsonObject> JsonObject);

	// Convert a JSON value to a string.
	FString JsonToString(const TSharedRef<FJsonValue> JsonValue);
}