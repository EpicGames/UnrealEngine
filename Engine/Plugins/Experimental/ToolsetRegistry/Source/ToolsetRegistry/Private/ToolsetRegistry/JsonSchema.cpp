// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/JsonSchema.h"

#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ToolsetRegistry::Internal
{
	namespace
	{
		const FString JsonSchemaDescriptionFieldName = TEXT("description");
		const FString JsonSchemaTypeFieldName = TEXT("type");
		const FString JsonSchemaPropertiesFieldName = TEXT("properties");

		// Maps JSON library types to valid JSON schema types.
		const struct {
			EJson Type;
			FString Name;
		} JsonSchemaTypeNames[] = {
			{EJson::Array, TEXT("array")},
			{EJson::Boolean, TEXT("boolean")},
			{EJson::Null, TEXT("null")},
			{EJson::Number, TEXT("number")},
			{EJson::Object, TEXT("object")},
			{EJson::String, TEXT("string")},
		};

		// Find a JSON schema type name given a JSON library type.
		FString FindJsonSchemaTypeName(EJson Type)
		{
			check(Type != EJson::None);  // EJson::None is not supported.
			FString Name;
			for (int I = 0; I < UE_ARRAY_COUNT(JsonSchemaTypeNames); ++I)
			{
				const auto& JsonSchemaTypeName = JsonSchemaTypeNames[I];
				if (Type == JsonSchemaTypeName.Type)
				{
					Name = JsonSchemaTypeName.Name;
					break;
				}
			}
			return Name;
		}
	}

	TSharedRef<FJsonObject> CreateJsonSchema(
		const FString& Description, EJson Type,
		const TSharedPtr<FJsonObject> Properties)
	{
		FString TypeName = FindJsonSchemaTypeName(Type);
		check(!TypeName.IsEmpty());
		check(!Properties || Type == EJson::Object);
		TSharedRef<FJsonObject> Schema = MakeShared<FJsonObject>();
		Schema->SetStringField(JsonSchemaDescriptionFieldName, Description);
		Schema->SetStringField(JsonSchemaTypeFieldName, TypeName);
		if (Properties)
		{
			Schema->SetObjectField(JsonSchemaPropertiesFieldName, Properties);
		}
		return Schema;
	}

}  // namespace UE::ToolsetRegistry::Internal