// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonTypes.h"
#include "Templates/SharedPointer.h"

namespace UE::ToolsetRegistry::Internal
{
	// Create a minimal JSON schema.
	//
	// See https://json-schema.org/
	//
	// This is intended to be used to create very simple schemas, if we need something that
	// supports building arbitrary schemas we should probably turn this into a object that
	// implements a builder pattern.
	//
	// Users still need to create constraints like:
	// * Set the "items" or "prefixItems" field for arrays.
	// * Manually create the properties object to define object fields.
	//
	// @param Description is the description of the schema.
	// @param Type is the schema's type.
	// @param Properties specifies the schemas for object fields when Type is EJson::Object.
	//
	// @returns JSON schema object.
	TSharedRef<FJsonObject> CreateJsonSchema(
		const FString& Description, EJson Type,
		const TSharedPtr<FJsonObject> Properties = TSharedPtr<FJsonObject>());
}  // namespace UE::ToolsetRegistry::Internal