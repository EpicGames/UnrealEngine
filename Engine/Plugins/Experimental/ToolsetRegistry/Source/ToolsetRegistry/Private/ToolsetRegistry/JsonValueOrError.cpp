// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/JsonValueOrError.h"

#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"

#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/JsonSchema.h"

namespace UE::ToolsetRegistry
{
	// Default descriptor for a FJsonValueOrError.
	const FJsonValueOrErrorSchemaDescriptor FJsonValueOrError::DefaultDescriptor = {
		TEXT("Either a value if successful or an error if the operation failed."),
		TEXT("returnValue"),
		TEXT("Always set to null on success."),
		TEXT("error"),
		TEXT("If set, describes the error that occurred."),
	};

	TSharedRef<FJsonObject> FJsonValueOrError::ToJsonObject(
		const FJsonValueOrErrorSchemaDescriptor& Descriptor) const
	{
		TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		if (HasError())
		{
			JsonObject->SetStringField(Descriptor.ErrorFieldName, GetError());
		}
		else
		{
			// If there is no error, this object must have a value.
			check(HasValue());
			TSharedPtr<FJsonValue> JsonValue = GetValue();
			// If there is no value return null.
			JsonObject->SetField(
				Descriptor.ValueFieldName,
				JsonValue ? JsonValue : MakeShared<FJsonValueNull>());
		}
		return JsonObject;
	}

	FString FJsonValueOrError::ToJsonString(
		const FJsonValueOrErrorSchemaDescriptor& Descriptor) const
	{
		return UE::ToolsetRegistry::Internal::JsonToString(ToJsonObject(Descriptor));
	}

	TSharedRef<FJsonObject> FJsonValueOrError::GetJsonSchema(
		TSharedPtr<FJsonObject> ValueSchema,
		const FJsonValueOrErrorSchemaDescriptor& Descriptor)
	{
		// Map the value and error field names to their schemas.
		TSharedRef<FJsonObject> Properties = MakeShared<FJsonObject>();
		Properties->SetObjectField(
			Descriptor.ValueFieldName,
			// Use "null" as the value's type if it has no schema.
			ValueSchema
			? ValueSchema
			: Internal::CreateJsonSchema(Descriptor.NullValueFieldDescription, EJson::Null));
		Properties->SetObjectField(
			Descriptor.ErrorFieldName,
			Internal::CreateJsonSchema(
				Descriptor.ErrorFieldDescription, EJson::String));

		// Create the schema for the overall object.
		TSharedRef<FJsonObject> Schema =
			Internal::CreateJsonSchema(
				Descriptor.ObjectDescription, EJson::Object, Properties);
		return Schema;
	}

	FJsonValueOrError FJsonValueOrError::FromJsonStringOrError(
		const TValueOrError<FString, FString>& JsonStringOrError)
	{
		if (JsonStringOrError.HasError())
		{
			return MakeError(JsonStringOrError.GetError());
		}
		check(JsonStringOrError.HasValue());
		const FString& JsonValueString = JsonStringOrError.GetValue();
		TSharedPtr<FJsonValue> JsonValue =
			UE::ToolsetRegistry::Internal::JsonStringToJsonValue(JsonValueString);
		if (!JsonValue)
		{
			return MakeError(
				FString::Printf(TEXT("Failed to parse JSON result '%s'"), *JsonValueString));
		}
		return MakeValue(JsonValue); 
	}

}  // namespace UE::ToolsetRegistry