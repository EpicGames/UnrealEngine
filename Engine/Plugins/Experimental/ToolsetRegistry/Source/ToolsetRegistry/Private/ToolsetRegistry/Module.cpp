// Copyright Epic Games, Inc. All Rights Reserved.


#include "ToolsetRegistry/Module.h"

#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FToolsetRegistryModule"


//
//  FToolsetRegistryModule
//


/*virtual*/ void FToolsetRegistryModule::StartupModule()
{
}


/*virtual*/ void FToolsetRegistryModule::ShutdownModule()
{
	
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FToolsetRegistryModule, ToolsetRegistry);
DEFINE_LOG_CATEGORY(LogToolsetRegistry);