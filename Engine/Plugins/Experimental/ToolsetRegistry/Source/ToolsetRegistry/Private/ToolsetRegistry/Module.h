// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"

//
// FToolsetRegistryModule
//
DECLARE_LOG_CATEGORY_EXTERN(LogToolsetRegistry, Log, Log);

class FToolsetRegistryModule : public IModuleInterface
{
public:
	
	// IModuleInterface interface 
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
};
