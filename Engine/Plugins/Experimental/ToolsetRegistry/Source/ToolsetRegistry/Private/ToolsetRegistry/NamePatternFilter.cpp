// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/NamePatternFilter.h"

namespace UE::ToolsetRegistry::Internal
{
	FRegexPattern MakeNamePattern(const FString& Pattern)
	{
		if (Pattern.Len() > 2 && Pattern.StartsWith(TEXT("/")) && Pattern.EndsWith(TEXT("/")))
		{
			return FRegexPattern(Pattern.Mid(1, Pattern.Len() - 2));
		}
		return FRegexPattern(TEXT("(?i)\\Q") + Pattern + TEXT("\\E"));
	}

	TArray<FRegexPattern> CompilePatterns(const TArray<FString>& PatternStrings)
	{
		TArray<FRegexPattern> Compiled;
		Compiled.Reserve(PatternStrings.Num());
		for (const FString& P : PatternStrings)
		{
			Compiled.Add(MakeNamePattern(P));
		}
		return Compiled;
	}

	bool IsNameAllowed(
		const FString& Name,
		const TArray<FRegexPattern>& BlockPatterns,
		const TArray<FRegexPattern>& AllowPatterns)
	{
		// Block always wins.
		for (const FRegexPattern& Pattern : BlockPatterns)
		{
			FRegexMatcher Matcher(Pattern, Name);
			if (Matcher.FindNext())
			{
				return false;
			}
		}
		// If an allow-list is set, the name must match at least one entry to be visible.
		for (const FRegexPattern& Pattern : AllowPatterns)
		{
			FRegexMatcher Matcher(Pattern, Name);
			if (Matcher.FindNext())
			{
				return true;
			}
		}
		return AllowPatterns.IsEmpty();
	}
}
