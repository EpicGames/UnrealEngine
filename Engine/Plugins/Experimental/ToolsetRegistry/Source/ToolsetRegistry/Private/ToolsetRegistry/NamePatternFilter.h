// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Regex.h"

namespace UE::ToolsetRegistry::Internal
{
	/// Compile a name-matching pattern. Patterns enclosed in forward slashes
	/// (e.g., /^Foo.*/) are treated as regular expressions; all other patterns are
	/// matched as case-insensitive substrings.
	FRegexPattern MakeNamePattern(const FString& Pattern);

	/// Compile a list of pattern strings into FRegexPattern objects.
	TArray<FRegexPattern> CompilePatterns(const TArray<FString>& PatternStrings);

	/// Returns true if Name passes the given block/allow filters. Block always wins.
	/// An empty allow-list permits any name that isn't blocked; a non-empty allow-list
	/// requires the name to match at least one allow pattern.
	bool IsNameAllowed(
		const FString& Name,
		const TArray<FRegexPattern>& BlockPatterns,
		const TArray<FRegexPattern>& AllowPatterns);
}
