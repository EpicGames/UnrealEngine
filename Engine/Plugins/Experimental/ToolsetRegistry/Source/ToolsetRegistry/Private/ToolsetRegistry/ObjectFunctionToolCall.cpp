// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ObjectFunctionToolCall.h"

#include "Async/Async.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/EngineTypes.h"
#include "Misc/Optional.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Templates/UnrealTemplate.h"
#include "ToolsetJson.h"
#include "UObject/FieldIterator.h"

#include "ToolsetRegistry/PropertyAccessors.h"
#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/ToolCallAsyncResult.h"
#include "ToolsetRegistry/ToolCallAsyncResultFutureHandler.h"
#include "ToolsetRegistry/ValueOrErrorFuture.h"

namespace UE::ToolsetRegistry
{
	using namespace UE::ToolsetRegistry::Internal;

	namespace
	{
		TSharedRef<FJsonObject> FunctionInputParamsJsonToValidJsonObject(
			const TOptional<FObjectFunctionToolCall::FFunctionInputParamsJson>&
				FunctionInputParamsJson)
		{
			if (FunctionInputParamsJson.IsSet())
			{
				if (FunctionInputParamsJson->IsType<TSharedPtr<FJsonObject>>())
				{
					// Return a copy of the JSON object, since it may be altered, and we 
					// don't want to alter the original.
					const TSharedPtr<FJsonObject> OriginalJsonObject =
						FunctionInputParamsJson->Get<TSharedPtr<FJsonObject>>();
					return OriginalJsonObject
						? MakeShared<FJsonObject>(*OriginalJsonObject)
						: MakeShared<FJsonObject>();
				}
				else
				{
					return JsonObjectOrEmpty(
						JsonStringToJsonObject(FunctionInputParamsJson->Get<FString>()));
				}
			}

			return MakeShared<FJsonObject>();
		}

		UWorld* GetWorld()
		{
			if (GEngine)
			{
				for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
				{
					if (WorldContext.WorldType == EWorldType::Game ||
						WorldContext.WorldType == EWorldType::PIE)
					{
						return WorldContext.World();
					}
				}
			}

			return nullptr;
		}

		void MaybeSetWorldContextPropertyInFrameMemory(const UFunction* Function,
			uint8* FrameMemory)
		{
			const FName WorldContextKey(TEXT("WorldContext"));
			const FName WorldContextParamName =
				Function->HasMetaData(WorldContextKey) ?
				FName(*Function->GetMetaData(WorldContextKey)) :
				FName(TEXT("WorldContextObject"));

			if (FProperty* WorldContextProperty =
				Function->FindPropertyByName(WorldContextParamName))
			{
				UWorld* World = GetWorld();
				FObjectPropertyBase* ObjectProperty =
					CastField<FObjectPropertyBase>(WorldContextProperty);
				if (ObjectProperty && World)
				{
					ObjectProperty->SetObjectPropertyValue_InContainer(FrameMemory, World);
				}
			}
		}

		/**
		  * Checks whether all the input parameters required by the function schema JSON are
		  * provided in the function input params JSON.
		  * @param FunctionDebugName Name of function, for debugging and error messages.
		  * @param FunctionInputParamsJsonObject The JSON option containing the function call
		  *     params.
		  * @param FunctionSchemaInputSchemaJsonObject The 'inputSchema' sub-JSON in the main
		  *     function schema JSON.
		  *	@param OutErrorString Output error string if a required param is missing, empty
		  *     otherwise.
		  *	@return Whether required params are all supplied.
		  */
		bool AreAllRequiredFunctionInputParamsProvided(
			const FString& FunctionDebugName,
			const TSharedRef<FJsonObject>& FunctionInputParamsJsonObject,
			const TSharedRef<FJsonObject>& FunctionSchemaInputSchemaJsonObject,
			FString* OutErrorString = nullptr)
		{
			// For each required param name, make sure we have a field for it in the function 
			// input params.
			{
				TArray<FString> RequiredParamNames;
				FunctionSchemaInputSchemaJsonObject->TryGetStringArrayField(
					TEXT("required"), RequiredParamNames);

				if (!RequiredParamNames.IsEmpty() &&
					FunctionInputParamsJsonObject->Values.IsEmpty())
				{
					// Caller error. Missing but expected params in incoming function 
					// input params JSON. 
					if (OutErrorString)
					{
						*OutErrorString = FString::Printf(TEXT(
							"Function \"%s\", "
							"input params are required by the function input schema Json, "
							"but incoming function input params Json is empty.\n"
							"Function schema Json -\n"
							"%s"),
							*FunctionDebugName,
							*JsonToString(FunctionSchemaInputSchemaJsonObject));
					}

					return false;
				}

				for (const FString& RequiredParamName : RequiredParamNames)
				{
					if (!FunctionInputParamsJsonObject->HasField(*RequiredParamName))
					{
						// Caller error. A missing but expected param in incoming function 
						// input params JSON.
						if (OutErrorString)
						{
							*OutErrorString = FString::Printf(TEXT(
								"Function \"%s\", "
								"input param \"%s\" is required by the function input schema "
								"Json, but is missing from the incoming function input params "
								"Json.\n"
								"Function schema Json -\n"
								"%s\n"
								"Function input params Json -\n"
								"%s"),
								*FunctionDebugName,
								*RequiredParamName,
								*JsonToString(FunctionSchemaInputSchemaJsonObject),
								*JsonToString(FunctionInputParamsJsonObject));
						}

						return false;
					}
				}
			}

			return true;
		}

		/**
		  * For each input schema property, if we don't have a field for it in the
		  * function input params, then add it to the function input params, and
		  * give it the same value as the default value of the corresponding function
		  * input schema property.
		  * (Note - Checked that we have all the required params first, so we know those
		  * will all be in the function input params JSON. Then this ends up only handling
		  * params that are both non-required and missing from the function input param
		  * JSON. To apply default values, we need a 'default' value to exist for each
		  * corresponding param object in the function input schema.)
		  * @param FunctionDebugName Name of the function, used for error message if needed.
		  * @param FunctionInputParamsJsonObject The JSON object containing the function call
		  *     params.
		  * @param FunctionSchemaInputSchemaJsonObject The 'inputSchema' sub-JSON in the main
		  *     function schema JSON.
		  * @param OutErrorString Error message, if function fails.
		  * @return Whether this function could provide defaults for all missing function input params.
		 */
		bool ProvideDefaultsForMissingFunctionInputParams(
			const FString& FunctionDebugName,
			const TSharedRef<FJsonObject>& FunctionInputParamsJsonObject,
			const TSharedRef<FJsonObject>& FunctionSchemaInputSchemaJsonObject,
			FString* OutErrorString = nullptr)
		{
			if (const TSharedPtr<FJsonObject>* InputPropertiesJsonObject;
				FunctionSchemaInputSchemaJsonObject->TryGetObjectField(
					TEXT("properties"), InputPropertiesJsonObject))
			{
				for (const auto& Pair : (*InputPropertiesJsonObject)->Values)
				{
					const auto& InputParamPropertyName = Pair.Key;

					const TSharedPtr<FJsonObject>& InputPropertyJsonObject =
						Pair.Value->AsObject();
					if (!InputPropertyJsonObject)
					{
						// Caller error. Missing but expected Json for an input param 
						// in incoming function input params JSON. 
						if (OutErrorString)
						{
							*OutErrorString = FString::Printf(TEXT(
								"Function \"%s\", "
								"input param \"%s\" has no Json object field in input params.\n"
								"Function input params Json -\n"
								"%s"),
								*FunctionDebugName,
								*InputParamPropertyName,
								*JsonToString(FunctionInputParamsJsonObject));
						}

						return false;
					}

					if (!FunctionInputParamsJsonObject->HasField(*InputParamPropertyName))
					{
						const TSharedPtr<FJsonValue> InputPropertyDefaultJsonValue =
							InputPropertyJsonObject->TryGetField(TEXT("default"));

						if (!InputPropertyDefaultJsonValue)
						{
							// Caller error. Missing but expected Json for an input param's
							// default in incoming function input params JSON.
							if (OutErrorString)
							{
								*OutErrorString = FString::Printf(TEXT(
									"Function \"%s\", "
									"input param \"%s\" needs a default value."
									"Function input params Json -\n"
									"%s"),
									*FunctionDebugName,
									*InputParamPropertyName,
									*JsonToString(FunctionInputParamsJsonObject));
							}

							return false;
						}

						FunctionInputParamsJsonObject->SetField(*InputParamPropertyName,
							InputPropertyDefaultJsonValue);
					}
				}
			}

			return true;
		}
	}

	//
	// UE::ToolsetRegistry::FObjectFunctionToolCall
	//

	namespace
	{
		// True only if the object is live and is the current version of its type.
		// TWeakObjectPtr::Get() handles destruction but happily returns the old
		// object while it lingers post-reinstance - invoking through it crashes
		// in the scripting runtime whose bindings have already been freed.
		bool IsLiveAndCurrent(const UObject* Obj)
		{
			if (!Obj || Obj->HasAnyFlags(RF_NewerVersionExists))
			{
				return false;
			}
			const UClass* RelevantClass = nullptr;
			if (const UFunction* AsFunc = Cast<UFunction>(Obj))
			{
				RelevantClass = AsFunc->GetOwnerClass();
			}
			else if (const UClass* AsClass = Cast<UClass>(Obj))
			{
				RelevantClass = AsClass;
			}
			else
			{
				RelevantClass = Obj->GetClass();
			}
			return !RelevantClass || !RelevantClass->HasAnyClassFlags(CLASS_NewerVersionExists);
		}
	}

	FObjectFunctionToolCall::FObjectFunctionToolCall(
		TNotNull<UObject*> InInstanceObject, TNotNull<UFunction*> InFunction) :
		InstanceObject(InInstanceObject),
		Function(InFunction),
		FunctionName(InFunction->GetName())
	{
		FunctionSchemaJsonObject = ToolsetJson::StructToJsonSchema(InFunction);
		check(FunctionSchemaJsonObject);
	}

	const UFunction* FObjectFunctionToolCall::GetFunction() const
	{
		UFunction* F = Function.Get();
		return IsLiveAndCurrent(F) ? F : nullptr;
	}

	TSharedPtr<FObjectFunctionToolCall> FObjectFunctionToolCall::Create(
		TNotNull<UObject*> InInstanceObject, TNotNull<UFunction*> InFunction)
	{
		return MakeShared<FObjectFunctionToolCall>(InInstanceObject, InFunction);
	}

	TSharedPtr<FObjectFunctionToolCall> FObjectFunctionToolCall::Create(
		TNotNull<UObject*> InInstanceObject, const FName& InFunctionName)
	{
		check(!InFunctionName.IsNone());
		UFunction* Function = InInstanceObject->GetClass()->FindFunctionByName(InFunctionName);
		check(Function);
		return Create(InInstanceObject, Function);
	}

	TFuture<FJsonValueOrError> FObjectFunctionToolCall::Execute(
		const TOptional<FFunctionInputParamsJson>& FunctionInputParamsJson,
		TSharedPtr<FToolCallExceptionHandler> ExceptionHandler) const
	{
		UFunction* LiveFunction = Function.Get();
		UObject* LiveInstance = InstanceObject.Get();
		if (!IsLiveAndCurrent(LiveFunction) || !IsLiveAndCurrent(LiveInstance))
		{
			return FJsonValueOrErrorFuture::MakeError(FString::Printf(
				TEXT("Function \"%s\" is no longer available."), *FunctionName));
		}

		// Get function input params JSON as a JSON object.
		TSharedPtr<FJsonObject> FunctionInputParamsJsonObject;
		FJsonValueOrError JsonValueOrError =
			BuildValidFunctionInputParamsJsonObject(FunctionInputParamsJson);
		if (JsonValueOrError.HasError())
		{
			return FJsonValueOrErrorFuture::Make(MoveTemp(JsonValueOrError));
		}
		FunctionInputParamsJsonObject = JsonValueOrError.GetValue()->AsObject();

		// Allocate the frame memory.
		uint8* FrameMemory;
		{
			// Invoke must use properties size, because it's allocating locals,
			// i.e. for script functions.
			const int32 FrameSize = LiveFunction->PropertiesSize;
			FrameMemory = static_cast<uint8*>(FMemory_Alloca_Aligned(FrameSize,
				LiveFunction->GetMinAlignment()));
			FMemory::Memzero(FrameMemory, FrameSize);
		}

		// We need to manually allocate each property in the frame memory.
		for (TFieldIterator<FProperty> It(LiveFunction); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			if (const FProperty* ParameterProp = *It;
				!ParameterProp->HasAnyPropertyFlags(CPF_ZeroConstructor))
			{
				ParameterProp->InitializeValue_InContainer(FrameMemory);
			}
		}

		// When we exit, we need to manually destroy each property in the frame memory.
		ON_SCOPE_EXIT
		{
			// Invoke will destroy locals, but caller owns parameters. This should also
			// destroy the result parameter.
			for (TFieldIterator<FProperty> It(LiveFunction);
				It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
			{
				It->DestroyValue_InContainer(FrameMemory);
			}
		};

		// TODO: A lot of this logic is duplicated from ScriptCore and should be ported to use
		// existing function invocation logic.

		// StackFrame must be set up before calling JsonDataToStruct so that errors raised via
		// RaiseScriptError during input validation can be reliably detected for C++ tool calls (Python tool
		// calls get a stack frame set up by the scripting runtime).
		FFrame StackFrame(LiveInstance, LiveFunction, FrameMemory, nullptr,
			LiveFunction->ChildProperties);

		// Convert JSON params into the function's frame memory. FrameMemory is null for
		// zero-parameter functions; with nothing to convert we shortcut to success.
		const bool bJsonConversionOk = !FrameMemory ||
			ToolsetJson::JsonDataToStruct(
				FunctionInputParamsJsonObject.ToSharedRef(), LiveFunction, FrameMemory);

		// Prefer the converter-raised script error (specific, e.g. "X is not a valid Y for
		// property 'Z'") over the generic conversion-failure path below. Param converters
		// raise via RaiseScriptError before returning false, so the exception handler holds
		// the actionable message whenever bJsonConversionOk is false.
		if (ExceptionHandler && !ExceptionHandler->GetException().IsEmpty())
		{
			FString ErrorMessage = ExceptionHandler->GetException();
			return FJsonValueOrErrorFuture::MakeError(
					FString::Printf(TEXT("Parameter error: %s."), *ErrorMessage));
		}

		if (!bJsonConversionOk)
		{
			// Caller error. Ill-formed incoming function input params JSON.
			return FJsonValueOrErrorFuture::MakeError(FString::Printf(TEXT(
				"Function \"%s\", "
				"could not convert incoming function input params Json to a UStruct. "
				"Incoming function input params Json may be incorrect or contain unsupported "
				"types.\n"
				"Function input params Json -\n"
				"%s"),
				*FunctionName,
				*JsonToString(FunctionInputParamsJsonObject.ToSharedRef())));
		}

		// In case this is a Blueprint function that has a world context object parameter,
		// give that parameter (in the frame memory) the World.
		MaybeSetWorldContextPropertyInFrameMemory(LiveFunction, FrameMemory);

		// If the function has out parameters, fill the stack frame's out parameter info with the
		// info for those params.
		if (LiveFunction->HasAnyFunctionFlags(FUNC_HasOutParms))
		{
			FOutParmRec** LastOut = &StackFrame.OutParms;
			for (FProperty* Property = (FProperty*)(LiveFunction->ChildProperties);
				Property && (Property->PropertyFlags & (CPF_Parm)) == CPF_Parm;
				Property = (FProperty*)Property->Next)
			{
				// This is used for optional parameters - the destination address for out parameter
				// values is  the address of the calling function so we'll need to know which
				// address to use if we need to evaluate the default param value expression located
				// in the new function's bytecode.
				if (Property->HasAnyPropertyFlags(CPF_OutParm))
				{
					CA_SUPPRESS(6263)
					FOutParmRec* Out = (FOutParmRec*)FMemory_Alloca(sizeof(FOutParmRec));
					// set the address and property in the out param info
					// note that since C++ doesn't support "optional out" we can ignore that here
					Out->PropAddr = Property->ContainerPtrToValuePtr<uint8>(FrameMemory);
					Out->Property = Property;

					// add the new out param info to the stack frame's linked list
					if (*LastOut)
					{
						(*LastOut)->NextOutParm = Out;
						LastOut = &(*LastOut)->NextOutParm;
					}
					else
					{
						*LastOut = Out;
					}
				}
			}

			// set the next pointer of the last item to NULL to mark the end of the list
			if (*LastOut)
			{
				(*LastOut)->NextOutParm = NULL;
			}
		}

		const bool bHasReturnParam = (LiveFunction->ReturnValueOffset != MAX_uint16);
		uint8* ReturnValueAddress = (bHasReturnParam ?
			(FrameMemory + LiveFunction->ReturnValueOffset) : nullptr);
		LiveFunction->Invoke(LiveInstance, StackFrame, ReturnValueAddress);

		FProperty* ReturnProperty = LiveFunction->GetReturnProperty();

		auto MaybeToolCallAsyncResult =
			ReturnProperty
			? PropertyValueAsObject<UToolCallAsyncResult>(ReturnProperty, ReturnValueAddress)
			: TOptional<TObjectPtr<UToolCallAsyncResult>>();
		// If the return value is an async result, return a future that completes when the result
		// completes.
		if (MaybeToolCallAsyncResult.IsSet())
		{
			if (!MaybeToolCallAsyncResult.GetValue())
			{
				return FJsonValueOrErrorFuture::MakeError(
					FString::Printf(
						TEXT("%s failed to return a UToolCallAsyncResult instance."),
						*FunctionName));
			}
			TStrongObjectPtr<UToolCallAsyncResultFutureHandler>
				ToolCallAsyncResultFutureHandler =
					UToolCallAsyncResultFutureHandler::Create(*MaybeToolCallAsyncResult);
			return ToolCallAsyncResultFutureHandler->GetValueAsJson().Next(
				[This = SharedThis(this), ToolCallAsyncResultFutureHandler](
					UE::ToolsetRegistry::FJsonValueOrError&& ValueOrError) -> FJsonValueOrError
				{
					return This->MakeResult(MoveTemp(ValueOrError));
				});
		}

		// Get the return value, if we have one.
		TSharedPtr<FJsonValue> ReturnJsonValue =
			ReturnProperty
			? ToolsetJson::PropertyToJsonData(ReturnProperty, ReturnValueAddress)
			: MakeShared<FJsonValueNull>();
		return FJsonValueOrErrorFuture::Make(MakeResult(MakeValue(ReturnJsonValue)));
	}

	FJsonValueOrError FObjectFunctionToolCall::MakeResult(FJsonValueOrError&& ValueOrError) const
	{
		if (ValueOrError.HasError()) return MoveTemp(ValueOrError);
		
		// If the value to return is invalid, return an error message instead of the return object.
		TSharedPtr<FJsonValue> JsonValue = ValueOrError.GetValue();
		if (!JsonValue.IsValid())
		{
			return MakeError(
				FString::Printf(
					TEXT("Function \"%s\": could not convert return value to JSON."),
					*FunctionName));
		}

		// According to the output schema: all returned values are required to be wrapped in a
		// "returnValue" object.
		TSharedPtr<FJsonObject> ReturnJsonObject = MakeShared<FJsonObject>();
		ReturnJsonObject->SetField(TEXT("returnValue"), JsonValue);
		return MakeValue(MakeShared<FJsonValueObject>(ReturnJsonObject));
	}

	FJsonValueOrError FObjectFunctionToolCall::BuildValidFunctionInputParamsJsonObject(
		const TOptional<FFunctionInputParamsJson>& FunctionInputParamsJson) const
	{
		// Get incoming function params as JSON object.
		const TSharedRef<FJsonObject> FunctionInputParamsJsonObject =
			FunctionInputParamsJsonToValidJsonObject(FunctionInputParamsJson);

		// With function schema JSON -
		// (1) Validate the function input JSON against the function schema's input schema JSON.
		// (2) Set any missing params in the function input params JSON to the defaults from 
		//		function schema's input schema JSON.
		if (const TSharedPtr<FJsonObject>* FunctionSchemaInputSchemaJsonObjectPtr;
			FunctionSchemaJsonObject->TryGetObjectField(
				TEXT("inputSchema"), FunctionSchemaInputSchemaJsonObjectPtr))
		{
			// TODO - We should ultimately be fully validating the function input params JSON
			// against the function schema's input schema JSON instead of this. This isn't
			// validating type compatibility, only checking existence of params.
			if (FString ErrorString;
				!AreAllRequiredFunctionInputParamsProvided(
					FunctionName,
					FunctionInputParamsJsonObject,
					FunctionSchemaInputSchemaJsonObjectPtr->ToSharedRef(),
					&ErrorString))
			{
				return MakeError(ErrorString);
			}

			if (FString ErrorString;
				!ProvideDefaultsForMissingFunctionInputParams(
					FunctionName,
					FunctionInputParamsJsonObject,
					FunctionSchemaInputSchemaJsonObjectPtr->ToSharedRef(),
					&ErrorString))
			{
				return MakeError(ErrorString);
			}
		}

		return MakeValue(MakeShared<FJsonValueObject>(FunctionInputParamsJsonObject));
	}
}

