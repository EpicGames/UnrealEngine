// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Dom/JsonObject.h"
#include "Misc/NotNull.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"

//
// UE::ToolsetRegistry::FObjectFunctionToolCall
//

namespace UE::ToolsetRegistry
{
	class FObjectFunctionToolCall :
		public TSharedFromThis<FObjectFunctionToolCall, ESPMode::ThreadSafe>
	{
	public:
		// Use Create() instead of constructing directly.
		FObjectFunctionToolCall(
			TNotNull<UObject*> InInstanceObject, TNotNull<UFunction*> InFunction);

		/**
		 * Function input params.
		 * Holds either a JSON object or a JSON string.
		 */
		using FFunctionInputParamsJson = TVariant<TSharedPtr<FJsonObject>, FString>;
		
		/**
		 * Create with a UFunction on a UObject instance. 
		* @param InInstanceObject UObject containing the function.
		 * @param InFunction UFunction contained in the UObject.
		 */
		static TSharedPtr<FObjectFunctionToolCall> Create(
			TNotNull<UObject*> InInstanceObject, TNotNull<UFunction*> InFunction);
		
		/**
		 * Create with a UFunction on a UObject instance, but using the UObject's UClass 
		 * where the function can be found by name.
		 * @param InInstanceObject UObject containing the function.
		 * @param InFunctionName UFunction name contained in the UObject.
		 * @return A function call object.
		 */
		static TSharedPtr<FObjectFunctionToolCall> Create(
			TNotNull<UObject*> InInstanceObject, const FName& InFunctionName);
		
		/**
		 * Given a JSON containing input parameters for a tool call, execute the function with those 
		 * input parameters.
		 * @param FunctionInputParamsJson The optional JSON object or string containing the
		 *   function parameters. If the function this class was initialized with has required
		 *   params, then this must be supplied with JSON that provides those required params.
		 *   Otherwise, an error will result. If the function this class was initialized with has
		 *   no required params, then this does not need to be supplied.
		 * @return Tool call result future. (See above). When tool result does not have an error -
 		 *   Returned objects will either be (1) the object itself, or (2) a value named 
 		 *   'returnValue' inside an object. And this may be null JSON value if (1) function 
		 *   does not return a value, or (2) if function does return a value and that return
		 *   value is null.
		 */
		TFuture<FJsonValueOrError> Execute(
			const TOptional<FFunctionInputParamsJson>& FunctionInputParamsJson = 
			TOptional<FFunctionInputParamsJson>(),
			TSharedPtr<FToolCallExceptionHandler> ExceptionHandler = nullptr) const;

		/**
		 * Returns the wrapped UFunction, or nullptr if it has been destroyed or
		 * superseded by a newer version (reinstanced class, reloaded package).
		 */
		const UFunction* GetFunction() const;

	private:
		FJsonValueOrError BuildValidFunctionInputParamsJsonObject(
			const TOptional<FFunctionInputParamsJson>& FunctionInputParamsJson) const;

		FJsonValueOrError MakeResult(FJsonValueOrError&& ValueOrError) const;

		TWeakObjectPtr<UObject> InstanceObject;
		TWeakObjectPtr<UFunction> Function;

		// Cached so error messages still work after the UFunction has been destroyed.
		FString FunctionName;

		TSharedPtr<FJsonObject> FunctionSchemaJsonObject;
	};
}
