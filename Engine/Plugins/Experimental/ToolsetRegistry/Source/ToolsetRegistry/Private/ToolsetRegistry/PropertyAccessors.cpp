// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/PropertyAccessors.h"

#include "Misc/Optional.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"

namespace UE::ToolsetRegistry::Internal
{
	TOptional<TObjectPtr<UObject>> PropertyValueAsObject(
		TNotNull<UClass*> Class, TNotNull<FProperty*> Property, TNotNull<const void*> ValueAddress)
	{
		TOptional<TObjectPtr<UObject>> Object;
		if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property);
			ObjectProperty && ObjectProperty->PropertyClass &&
			ObjectProperty->PropertyClass->IsChildOf(Class))
		{
			Object.Emplace(ObjectProperty->GetObjectPropertyValue(ValueAddress));
		}
		return Object;
	}
}