// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotNull.h"
#include "Misc/Optional.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"

namespace UE::ToolsetRegistry::Internal
{
	// Get a property value as an object of a specific type.
	// The returned optional is populated if the property matches the specified UClass.
	TOptional<TObjectPtr<UObject>> PropertyValueAsObject(
		TNotNull<UClass*> Class, TNotNull<FProperty*> Property,
		TNotNull<const void*> ValueAddress);

	// Get a property value as an object of a specific type.
	// The returned optional is populated if the property matches the UClass ObjectT.
	template<typename ObjectT>
	TOptional<TObjectPtr<ObjectT>> PropertyValueAsObject(
		TNotNull<FProperty*> Property, TNotNull<const void*> ValueAddress)
	{
		TOptional<TObjectPtr<ObjectT>> Object;
		auto MaybeObjectBase = PropertyValueAsObject(
			TNotNull<UClass*>(ObjectT::StaticClass()), Property, ValueAddress);
		if (MaybeObjectBase.IsSet()) Object.Emplace(Cast<ObjectT>(*MaybeObjectBase));
		return Object;
	}
}