// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/PythonTestRunner.h"


#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "Dom/JsonValue.h"
#include "Engine/Engine.h"
#include "IPythonScriptPlugin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"
#include "PythonScriptTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonTypes.h"
#include "Templates/UnrealTemplate.h"

#include "ToolsetRegistry/Module.h"

namespace
{
	// Get the initialized Python script plugin.
	IPythonScriptPlugin* GetPythonScriptPlugin()
	{
		IPythonScriptPlugin* Python = IPythonScriptPlugin::Get();
		if (!(Python && Python->IsPythonInitialized()))
		{
			return nullptr;
		}
		return Python;
	}
}

TValueOrError<void, FString> ValidatePythonString(
	const FString& StringToValidate, const FString& ErrorContext)
{
	static const TCHAR InvalidSubstring[] = TEXT("'");
	if (StringToValidate.Contains(InvalidSubstring))
	{
		return MakeError(
			FString::Printf(
				TEXT("%s failed, invalid substring `%s` found in `%s`"),
				*ErrorContext, InvalidSubstring, *StringToValidate));
	}
	return MakeValue();
}

TValueOrError<void, FString> ValidatePythonStrings(
	const TArray<FString>& StringsToValidate, const FString& ErrorContext)
{
	for (const FString& StringToValidate : StringsToValidate)
	{
		TValueOrError<void, FString> MaybeError =
			ValidatePythonString(StringToValidate, ErrorContext);
		if (MaybeError.HasError()) return MaybeError;
	}
	return MakeValue();
}

TValueOrError<FString, FString> FPythonTestRunnerSearchOptions::BuildPythonCode() const
{

	TFunction<TValueOrError<void, FString>()> Validators[] = {
		[this]() -> auto
		{
			return ValidatePythonString(
				RootModule, TEXT("Validating search option RootModule"));
		},
		[this]() -> auto
		{
			return ValidatePythonString(
				TestModuleGlob, TEXT("Validating search option TestModuleGlob"));
		},
		[this]() -> auto
		{
			return ValidatePythonStrings(
				TestNameGlobs, TEXT("Validating search option TestNameGlobs"));
		},
	};
	for (int i = 0; i < UE_ARRAY_COUNT(Validators); ++i)
	{
		TValueOrError<void, FString> MaybeError = Validators[i]();
		if (MaybeError.HasError()) return MakeError(MaybeError.StealError());
	}

	TArray<FString> Arguments;
	if (!RootModule.IsEmpty())
	{
		Arguments.Push(FString::Printf(TEXT("root_module='%s'"), *RootModule));
	}
	if (!TestModuleGlob.IsEmpty())
	{
		Arguments.Push(
			FString::Printf(TEXT("test_module_glob='%s'"), *TestModuleGlob));
	}
	if (!TestNameGlobs.IsEmpty())
	{
		TArray<FString> QuotedTestNameGlobs;
		Algo::Transform(
			TestNameGlobs, QuotedTestNameGlobs,
			[](const FString& Item) -> FString { return FString::Printf(TEXT("'%s'"), *Item); });
		FString Items = FString::Join(QuotedTestNameGlobs, TEXT(", "));
		Arguments.Push(
			FString::Printf(TEXT("test_name_globs=set([%s])"), *Items));
	}
	return MakeValue(
		FString::Printf(
			TEXT("unittest_runner.TestSearchOptions(%s)"),
			*FString::Join(Arguments, TEXT(", "))));
}

namespace UE::ToolsetRegistry
{
	FPythonAutomationTest::FPythonAutomationTest(
		const FString& Name, const FPythonTestRunnerSearchOptions& InSearchOptions,
		float InTimeoutInSeconds) :
		FAutomationTestBase(Name, false),
		SearchOptions(InSearchOptions),
		TimeoutInSeconds(InTimeoutInSeconds) {}
	FPythonAutomationTest::~FPythonAutomationTest() {}

	EAutomationTestFlags FPythonAutomationTest::GetTestFlags() const
	{
		return EAutomationTestFlags::EditorContext |
			EAutomationTestFlags::ProductFilter |
			EAutomationTestFlags::CriticalPriority;
	}

	FString FPythonAutomationTest::GetBeautifiedTestName() const { return GetTestName(); }

	uint32 FPythonAutomationTest::GetRequiredDeviceNum() const { return 1; }

	void FPythonAutomationTest::GetTests(
		TArray<FString>& OutBeautifiedNames,
		TArray <FString>& OutTestCommands) const
	{
		FString ErrorContext =
			FString::Printf(TEXT("Discovering %s tests"), *GetTestName());
		TValueOrError<FString, FString> MaybeSearchOptionsCode = SearchOptions.BuildPythonCode();
		if (MaybeSearchOptionsCode.HasError())
		{
			UE_LOGF(LogToolsetRegistry, Error, "%ls", *MaybeSearchOptionsCode.GetError());
			return;
		}
		TValueOrError<TArray<FString>, FString> MaybeTestIds = RunAndParseTestCaseIds(
			FString::Printf(
				TEXT("unittest_runner.discover_tests(search_options=%s)"),
				*MaybeSearchOptionsCode.GetValue()),
			ErrorContext);
		if (MaybeTestIds.HasError())
		{
			UE_LOGF(LogToolsetRegistry, Error, "%ls", *MaybeTestIds.GetError());
			return;
		}
		TArray<FString> TestIds = MaybeTestIds.StealValue();
		if (TestIds.Num() == 0)
		{
			UE_LOGF(
				LogToolsetRegistry, Error, "%ls failed, no tests found for search %ls",
				*ErrorContext, *MaybeSearchOptionsCode.GetValue());
			return;
		}
		for (const FString& TestId : TestIds)
		{
			OutBeautifiedNames.Add(TestId);
			OutTestCommands.Add(TestId);
		}
	}

	bool FPythonAutomationTest::RunTest(const FString& Parameters)
	{
		auto MaybeTestId = StartTestExecution(Parameters);
		if (MaybeTestId.HasError())
		{
			UE_LOGF(LogToolsetRegistry, Error, "%ls", *MaybeTestId.GetError());
			return false;
		}

		if (!PollForTestCompletion(MaybeTestId.GetValue()))
		{
			AddCommand(
				new FUntilCommand(
					[This = AsShared(), TestId = MaybeTestId.StealValue()]() -> bool
					{
						return This->PollForTestCompletion(TestId);
					},
					TimeoutInSeconds));
		}
		return true;
	}

	TValueOrError<FString, FString> FPythonAutomationTest::StartTestExecution(
		const FString& TestId)
	{
		FPythonTestRunnerSearchOptions SingleTestSearchOptions = SearchOptions;
		SingleTestSearchOptions.TestNameGlobs.Empty();
		SingleTestSearchOptions.TestNameGlobs.Add(TestId);

		FString ErrorContext =
			FString::Printf(TEXT("Running %s test %s"), *GetTestName(), *TestId);
		TValueOrError<FString, FString> MaybeSearchOptionsCode =
			SingleTestSearchOptions.BuildPythonCode();
		if (MaybeSearchOptionsCode.HasError())
		{
			return MakeError(MaybeSearchOptionsCode.StealError());
		}
		TValueOrError<TArray<FString>, FString> MaybeTestIds = RunAndParseTestCaseIds(
			FString::Printf(
				TEXT("unittest_runner.discover_and_run_tests(search_options=%s)[0]"),
				*MaybeSearchOptionsCode.GetValue()),
			ErrorContext);

		if (MaybeTestIds.HasError()) return MakeError(MaybeTestIds.StealError());

		TArray<FString> TestIds = MaybeTestIds.StealValue();
		if (TestIds.Num() != 1)
		{
			return MakeError(
				FString::Printf(
					TEXT("%s failed. Unexpected number of tests %d when searching with %s."),
					*ErrorContext, TestIds.Num(), *MaybeSearchOptionsCode.GetValue()));
		}
		return MakeValue(TestIds[0]);
	}

	bool FPythonAutomationTest::PollForTestCompletion(const FString& TestId)
	{
		// NOTE: When a test is executed via RunTest(), this method is called in the context
		// of FUntilCommand with a timeout configured.
		TValueOrError<TOptional<bool>, FString> MaybeResult = GetLastTestResult(TestId);
		if (MaybeResult.HasError())
		{
			UE_LOGF(LogToolsetRegistry, Error, "%ls", *MaybeResult.GetError());
			return true /* stop polling */;
		}
		check(MaybeResult.HasValue());
		TOptional<bool> Result = MaybeResult.StealValue();
		if (Result.IsSet())
		{
			TestTrue(TEXT("Test succeeded"), Result.GetValue());
			return true /* stop polling */;
		}
		return false; /* continue polling */;
	}

	TValueOrError<TOptional<bool>, FString> FPythonAutomationTest::GetLastTestResult(
		const FString& TestId) const
	{
		const FString ErrorContext =
			FString::Printf(TEXT("Getting %s test %s result"), *GetTestName(), *TestId);
		TValueOrError<void, FString> MaybeTestIdError =
			ValidatePythonString(TestId, TEXT("Validating test ID"));
		if (MaybeTestIdError.HasError()) return MakeError(MaybeTestIdError.StealError());

		TValueOrError<FString, FString> StringResult = RunTestRunnerPythonCode(
			FString::Printf(
				TEXT("unittest_runner.get_last_test_result('%s')"),
				*TestId),
			ErrorContext);
		if (StringResult.HasError()) return MakeError(StringResult.StealError());

		TOptional<bool> Result;
		FString StringResultValue = StringResult.StealValue();
		// Still executing?
		if (StringResultValue != TEXT("None")) Result.Emplace(StringResultValue.ToBool());
		return MakeValue(Result);
	}

	TValueOrError<TArray<FString>, FString> FPythonAutomationTest::ParseJsonStringArray(
		const FString& StringArrayJson)
	{
		TArray<TSharedPtr<FJsonValue>> JsonParsed;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(StringArrayJson);
		if (!FJsonSerializer::Deserialize(Reader, JsonParsed))
		{
			return MakeError(
				FString::Printf(TEXT("Failed to parse JSON list '%s'"), *StringArrayJson));
		}
		TArray<FString> Result;
		for (const auto& Value : JsonParsed)
		{
			if (Value->Type != EJson::String)
			{
				return MakeError(
					FString::Printf(
						TEXT("Unexpected non-string value found in JSON array '%s'"),
						*StringArrayJson));
			}
			Result.Add(Value->AsString());
		}
		return MakeValue(MoveTemp(Result));
    }

	TValueOrError<FString, FString> FPythonAutomationTest::RunTestRunnerPythonCode(
		const FString& CodeToRun, const FString& ErrorContext)
	{
		IPythonScriptPlugin* PythonScriptPlugin = GetPythonScriptPlugin();
		if (!PythonScriptPlugin)
		{
			return MakeError(
				FString::Printf(TEXT("%s failed, Python not initialized"), *ErrorContext));
		}

		FPythonCommandEx Command;
		Command.Command =
			FString::Printf(
				TEXT(
					"(exec('import json; from toolset_registry.tests import unittest_runner') "
					"or %s)"),
				*CodeToRun);
		Command.ExecutionMode = EPythonCommandExecutionMode::EvaluateStatement;
		if (!PythonScriptPlugin->ExecPythonCommandEx(Command))
		{
			return MakeError(
				FString::Printf(
					TEXT("%s failed, Python failed to execute '%s'"),
					*ErrorContext, *Command.Command));
		}
		// The Python Script plugin uses repr to convert strings to strings which quotes them with
		// single quotes, so remove quotation here.
		bool bIgnoredCharRemoved;
		Command.CommandResult.TrimCharInline(FString::ElementType('\''), &bIgnoredCharRemoved);
		return MakeValue(Command.CommandResult);
	}

	TValueOrError<TArray<FString>, FString> FPythonAutomationTest::RunAndParseTestCaseIds(
		const FString& FunctionThatReturnsTestCaseSequence,
		const FString& ErrorContext)
	{
		TValueOrError<FString, FString> Result =
			RunTestRunnerPythonCode(
				FString::Printf(
					TEXT("json.dumps([test.id() for test in %s])"),
					*FunctionThatReturnsTestCaseSequence),
				ErrorContext);
		if (Result.HasError()) return MakeError(Result.StealError());
		return ParseJsonStringArray(Result.StealValue());
	}
}

UPythonTestRunner::~UPythonTestRunner()
{
	Reset();
}

TArray<FString> UPythonTestRunner::GetTests() const
{
	TArray<FString> IgnoredNames;
	TArray<FString> TestIds;
	if (CheckInitialized()) Test->GetTests(IgnoredNames, TestIds);
	return TestIds;
}

void UPythonTestRunner::RunTest(const FString& TestId)
{
	if (CheckInitialized())
	{
		check(RunTestDelegate.IsSet());
		RunTestDelegate(FString::Printf(TEXT("%s.%s"), *Test->GetTestName(), *TestId));
	}
}

FPythonTestRunnerResult UPythonTestRunner::GetLastTestResult(const FString& TestId) const
{
	FPythonTestRunnerResult TestResult = { false, false };
	if (!CheckInitialized()) return TestResult;

	TValueOrError<TOptional<bool>, FString> MaybeResult = Test->GetLastTestResult(TestId);
	if (MaybeResult.HasError())
	{
		UE_LOGF(LogToolsetRegistry, Error, "%ls", *MaybeResult.GetError());
		return TestResult;
	}

	TOptional<bool> Result = MaybeResult.StealValue();
	if (!Result.IsSet()) return TestResult;
	TestResult.bIsComplete = true;
	TestResult.bIsSuccessful = Result.GetValue();
	return TestResult;
}

bool UPythonTestRunner::CheckInitialized() const
{
	if (!Test.IsValid())
	{
		UE_LOGF(LogToolsetRegistry, Error, "PythonTestRunner must be instanced using create()");
		return false;
	}
	return true;
}

void UPythonTestRunner::Initialize(
	const FString& BaseName,
	const FPythonTestRunnerSearchOptions& SearchOptions,
	float TimeoutInSeconds)
{
	Reset();
	Test = MakeShared<UE::ToolsetRegistry::FPythonAutomationTest>(
		BaseName, SearchOptions, TimeoutInSeconds);
	FAutomationTestFramework::Get().RegisterAutomationTest(Test->GetTestName(), Test.Get());
	RunTestDelegate =
		[](const FString& TestName) -> void
		{
			if (GEngine)
			{
				GEngine->Exec(
					GEngine->GetWorld(),
					*FString::Printf(TEXT("Automation RunTests %s"), *TestName));
			}
			else
			{
				UE_LOGF(
					LogToolsetRegistry, Error, "Unable to run test %ls, engine not available",
					*TestName);
			}
		};
	OnExitDelegateHandle = UE::ToolsetRegistry::FDelegateHandleRaii::Create(
		FCoreDelegates::OnEnginePreExit,
		FCoreDelegates::OnEnginePreExit.AddUObject(this, &UPythonTestRunner::Reset));
}

void UPythonTestRunner::Reset()
{
	if (Test.IsValid())
	{
		FAutomationTestFramework::Get().UnregisterAutomationTest(Test->GetTestName());
		Test.Reset();
	}
	RunTestDelegate.Reset();
	OnExitDelegateHandle.Reset();
}

UPythonTestRunner* UPythonTestRunner::Create(
	const FString& BaseName,
	const FPythonTestRunnerSearchOptions& SearchOptions,
	float TimeoutInSeconds)
{
	TObjectPtr<UPythonTestRunner> NewInstance = NewObject<UPythonTestRunner>();
	NewInstance->Initialize(BaseName, SearchOptions, TimeoutInSeconds);
	return NewInstance.Get();
}
