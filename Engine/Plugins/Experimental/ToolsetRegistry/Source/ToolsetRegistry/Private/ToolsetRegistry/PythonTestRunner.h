// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "ToolsetRegistry/DelegateHandle.h"

#include "PythonTestRunner.generated.h"

/// Search parameters for Python tests.
USTRUCT(BlueprintType, Category="PythonTestRunner")
struct FPythonTestRunnerSearchOptions
{
	GENERATED_BODY()

	/// See toolset_registry.tests.unittest_runner.TestSearchOptions
	/// Python module / package to search under.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="PythonTestRunner")
	FString RootModule;

	/// Glob used to match test module names.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="PythonTestRunner")
	FString TestModuleGlob;

	/// Globs used to match individual test names.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="PythonTestRunner")
	TArray<FString> TestNameGlobs;

	/// Generate code to construct toolset_registry.tests.unittest_runner.TestSearchOptions
	/// assuming test_runner is imported in the current scope.
	TValueOrError<FString, FString> BuildPythonCode() const;
};

// Used to access internals of FPythonAutomationTest in tests.
class FPythonAutomationTestAccessor;

namespace UE::ToolsetRegistry
{
	/// Discovers and runs Python tests.
	///
	/// At the moment this has the limitation that all Python tests with class or module level
	/// setup and teardown hooks (e.g setUpClass()) will be executed before and after every
	/// test case.
	class FPythonAutomationTest :
		public FAutomationTestBase,
		public TSharedFromThis<FPythonAutomationTest>
	{
		friend FPythonAutomationTestAccessor;

	public:
		/// Construct a Python automation test.
		/// 
		/// @param Name Base name of the test case (e.g My.Tests) which will be prefixed
		///   to all Python test case names in test automation view.
		/// @param SearchOptions Options used to discover Python test cases.
		/// @param TimeoutInSeconds Time to wait for each test case to execute.
		FPythonAutomationTest(
			const FString& Name,
			const FPythonTestRunnerSearchOptions& SearchOptions,
			float TimeoutInSeconds = 5.0f);
		virtual ~FPythonAutomationTest() override;

		virtual EAutomationTestFlags GetTestFlags() const override;
		virtual FString GetBeautifiedTestName() const override;
		virtual uint32 GetRequiredDeviceNum() const override;

		/// Discover Python tests.
		///
		/// @param OutBeautifiedNames Same as OutTestCommands.
		/// @param OutTestCommands Fully qualified names of Python tests discovered by this
		///   class.
		virtual void GetTests(
			TArray<FString>& OutBeautifiedNames,
			TArray<FString>& OutTestCommands) const override;

		/// Run requested Python test.
		///
		/// @param Parameters Test command to execute (see GetTests()).
		///
		/// @returns true if the test started, false otherwise.
		virtual bool RunTest(const FString& Parameters) override;

		/// Determine whether a test has finished executing after running it.
		///
		/// @param TestId ID of the test to query.
		///
		/// @returns If successful, returns an optional bool value that indicates whether the test
		///   is not started or running (not set), succeeded or failed.
		TValueOrError<TOptional<bool>, FString> GetLastTestResult(const FString& TestId) const;

	private:
		// Run a test without waiting for it to complete returning the ID of the test being
		// executed.
		TValueOrError<FString, FString> StartTestExecution(const FString& TestId);

		// Poll to determine whether a test is complete returning false to continue polling, true
		// to stop polling.
		bool PollForTestCompletion(const FString& TestId);

	private:
		FPythonTestRunnerSearchOptions SearchOptions;
		float TimeoutInSeconds;

	private:
		// Parse a JSON string list into a string array.
		static TValueOrError<TArray<FString>, FString> ParseJsonStringArray(
			const FString& StringArrayJson);

		// Run Python code in a context where json and unittest_runner modules are imported.
		static TValueOrError<FString, FString> RunTestRunnerPythonCode(
			const FString& CodeToRun, const FString& ErrorContext);

		// Run a function that returns unittest.TestCases returning IDs of the test cases.
		// In this context json and unittest_runner modules are imported.
		static TValueOrError<TArray<FString>, FString> RunAndParseTestCaseIds(
			const FString& FunctionThatReturnsTestCaseSequence,
			const FString& ErrorContext);
	};
}

/// Result of a Python test.
USTRUCT(BlueprintType, Category="PythonTestRunner")
struct FPythonTestRunnerResult
{
	GENERATED_BODY()

	/// Whether a queried test has been executed and is complete.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="PythonTestRunner")
	bool bIsComplete = false;

	/// Whether a queried test completed successfully.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="PythonTestRunner")
	bool bIsSuccessful = false;
};

/// Python automation test runner.
UCLASS(MinimalAPI, BlueprintType, Category="PythonTestRunner")
class UPythonTestRunner : public UObject
{
	// Used to access internals of this class in tests.
	friend class FPythonTestRunnerAccessor;

	GENERATED_BODY()

public:
	virtual ~UPythonTestRunner() override;

	/// Discover tests.
	///
	/// @returns Test IDs discovered by this runner.
	UFUNCTION(BlueprintCallable, Category="PythonTestRunner")
	TArray<FString> GetTests() const;

	/// Run a test.
	///
	/// @param TestId Python test ID to run. This must be a test ID returned by
	///    GetTests().
	UFUNCTION(BlueprintCallable, Category="PythonTestRunner")
	void RunTest(const FString& TestId);

	/// Get the result of a test.
	///
	/// @param TestId Python test ID to query.
	///
	/// @return Last result of the test.
	UFUNCTION(BlueprintCallable, Category="PythonTestRunner")
	FPythonTestRunnerResult GetLastTestResult(const FString& TestId) const;

private:
	// Ensure the instance is initialize otherwise log an error and return false.
	bool CheckInitialized() const;

	// Initialize the instance.
	void Initialize(
		const FString& BaseName,
		const FPythonTestRunnerSearchOptions& SearchOptions,
		float TimeoutInSeconds);

	// Clear the instance.
	void Reset();

private:
	TSharedPtr<UE::ToolsetRegistry::FPythonAutomationTest> Test;
	// Delegate that runs a test by name, this is overridable by tests so that it's possible
	// to test RunTests() since tests can't recursively call
	// FAutomationTestFramework::StartTestByName().
	TFunction<void(const FString&)> RunTestDelegate;

	UE::ToolsetRegistry::FDelegateHandleRaii OnExitDelegateHandle;

public:
	/// Create a test runner.
	///
	/// @param BaseName Base name of the test case (e.g My.Tests) which will be prefixed
	///   to all Python test case names in test automation view.
	/// @param SearchOptions Options used to discover Python test cases.
	/// @param TimeoutInSeconds Time to wait for each test case to execute.
	/// 
	/// @returns Instance of this object that holds a reference to a FPythonAutomationTest.
	///
	/// This is intended to be used from Python to register a set of Python unittest tests with the
	/// Unreal test runner.
	///
	/// For example, the following when called from a plugin's `init_unreal.py` script, will register
	/// tests under the package `foo.bar.tests` as tests underneath `MyPackage.Magic`:
	///
	/// @code{.py}
	/// import unreal
	/// from foo.bar import tests
	/// 
	/// # It is important to reference the runner in a global to prevent it from being garbage
	/// # collected when instanced from an init_unreal.py script.
	/// tests._test_runner = unreal.PythonTestRunner.create(
	///   'MyPackage.Magic',
	///   unreal.PythonTestRunnerSearchOptions(root_module=tests.__package__))
	/// @endcode
	///
	UFUNCTION(BlueprintCallable, Category="PythonTestRunner")
	static UPythonTestRunner* Create(
		const FString& BaseName,
		const FPythonTestRunnerSearchOptions& SearchOptions,
		float TimeoutInSeconds = 5.0f);
};
