// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceConverter.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Blueprint.h"
#include "JsonObjectConverter.h"
#include "JsonSchema/JsonSchemaGenerator.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/PackageName.h"
#include "Misc/StringOutputDevice.h"

#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/PropertyAccessors.h"
#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/ToolsetJson.h"

namespace
{
	bool StringMeansNull(const FString& String)
	{
		return String.IsEmpty() || String.Equals("None", ESearchCase::IgnoreCase) || String.Equals("null", ESearchCase::IgnoreCase);
	}

	// Returns the display name for a property in error messages.
	// Python optional params (e.g. 'unreal.Class | None') are implemented as an FOptionalProperty
	// wrapping an inner transient value property named 'TransientPythonProperty'. In that case,
	// return the outer optional property's name (the actual parameter name) instead.
	FString GetPropertyDisplayName(const FProperty* Property)
	{
		if (const FField* OwnerField = Property->Owner.ToField())
		{
			if (CastField<const FOptionalProperty>(OwnerField))
			{
				return OwnerField->GetName();
			}
		}
		return Property->GetName();
	}

	// Raises a script error with the given message and returns false, so each rejection site
	// in JsonDataToProperty collapses to `return RejectWith(FString::Printf(...));` instead of
	// a three-line RaiseScriptError + return pair. Takes a pre-built FString because
	// FString::Printf's format-string parameter must be a literal — forwarding it through a
	// template won't compile.
	bool RejectWith(const FString& Message)
	{
		UKismetSystemLibrary::RaiseScriptError(Message);
		return false;
	}

	// Helper to validate soft reference type without loading the asset
	bool ValidateSoftReferenceType(
		const FString& RefPath,
		const UClass* ExpectedClass,
		FString& OutErrorMessage)
	{
		// Try to get asset data from registry
		FAssetRegistryModule& AssetRegistryModule =
			FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		FSoftObjectPath SoftPath(RefPath);
		FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(SoftPath);

		if (!AssetData.IsValid())
		{
			// Asset not in registry - could be:
			// 1. Invalid path (asset doesn't exist)
			// 2. Not scanned yet
			// 3. Generated at runtime
			// We'll allow it and let it fail at load time if invalid
			return true; // Permissive: don't block unknown assets
		}

		// Get the asset's class
		UClass* AssetClass = AssetData.GetClass();
		if (!AssetClass)
		{
			// Can't determine class from registry
			return true; // Permissive
		}

		// Check if asset class is compatible with expected class
		if (!AssetClass->IsChildOf(ExpectedClass))
		{
			OutErrorMessage = FString::Printf(
				TEXT("%s is not valid %s (asset is %s) for property"),
				*RefPath,
				*ExpectedClass->GetName(),
				*AssetClass->GetName());
			return false;
		}

		return true;
	}
}

namespace UE::ToolsetRegistry
{
	FString FToolsetReferenceConverter::GetName() const
	{
		static FString Name(TEXT("ReferenceConverter"));
		return Name;
	}

	bool FToolsetReferenceConverter::CanConvertProperty(TNotNull<const FProperty*> Property)
	{
		if (CastField<FObjectProperty>(Property))
		{
			return true;
		}
		else if (
			CastField<FSoftObjectProperty>(Property) ||
			CastField<FWeakObjectProperty>(Property))
		{
			return true;
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			const UScriptStruct* ScriptStruct = StructProperty->Struct;
			if (ScriptStruct == FSoftObjectPath::StaticStruct() ||
				ScriptStruct == FSoftClassPath::StaticStruct())
			{
				return true;
			}
		}

		return false;
	}

	TSharedPtr<FJsonObject> FToolsetReferenceConverter::PropertyToJsonSchema(
		TNotNull<const FProperty*> Property)
	{
		TSharedPtr<FJsonObject> Schema = MakeShared< FJsonObject>();
		FJsonObject::Duplicate(
			FJsonSchemaGenerator::UStructToJsonSchemaObject<FToolsetReference>(), Schema);
		UClass* ClassType = nullptr;
		bool IsClass = false;
		if (const FClassProperty* ClassProperty = CastField<FClassProperty>(Property))
		{
			ClassType = ClassProperty->MetaClass;
			IsClass = true;
		}
		else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
		{
			ClassType = ObjectProperty->PropertyClass;
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			const UScriptStruct* ScriptStruct = StructProperty->Struct;
			if (ScriptStruct == FSoftObjectPath::StaticStruct())
			{
				ClassType = UObject::StaticClass();
			}
			else if (ScriptStruct == FSoftClassPath::StaticStruct())
			{
				ClassType = UObject::StaticClass();
				IsClass = true;
			}
		}
		else
		{
			check(false);
		}
		check(ClassType);
		FString TypeInfo = IsClass ? FString(TEXT("Class@")) : FString();
		TypeInfo += ClassType->GetPathName();
		Schema->SetStringField(TEXT("title"), TypeInfo);
		return Schema;
	}

	TSharedPtr<FJsonValue> FToolsetReferenceConverter::PropertyToDefault(
		TNotNull<const FProperty*> Property, const FString& DefaultString)
	{
		if (StringMeansNull(DefaultString))
		{
			return MakeShared<FJsonValueNull>();
		}
		else
		{
			FToolsetReference Reference;
			Reference.RefPath = DefaultString;
			TSharedPtr<FJsonObject> DefaultObject = MakeShared<FJsonObject>();
			DefaultObject = FJsonObjectConverter::UStructToJsonObject(Reference);
			return MakeShared<FJsonValueObject>(DefaultObject);
		}
	}

	TSharedPtr<FJsonValue> FToolsetReferenceConverter::PropertyToJsonData(
		TNotNull<FProperty*> Property, const void* Value)
	{
		FToolsetReference Reference;
		if (auto MaybeObject = Internal::PropertyValueAsObject<UObject>(Property, Value);
			MaybeObject.IsSet())
		{
			if (*MaybeObject)
			{
				Reference.RefPath = (*MaybeObject)->GetPathName();
			}
			else
			{
				return nullptr;
			}
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			const UScriptStruct* ScriptStruct = StructProperty->Struct;
			if (ScriptStruct == FSoftObjectPath::StaticStruct())
			{
				const FSoftObjectPath* SoftObjectPath = reinterpret_cast<const FSoftObjectPath*>(Value);
				Reference.RefPath = SoftObjectPath->ToString();
			}
			else if (ScriptStruct == FSoftClassPath::StaticStruct())
			{
				const FSoftClassPath* SoftClassPath = reinterpret_cast<const FSoftClassPath*>(Value);
				Reference.RefPath = SoftClassPath->ToString();
			}
			else
			{
				check(false);
			}
		}
		else
		{
			check(false);
		}
		return MakeShared<FJsonValueObject>(FJsonObjectConverter::UStructToJsonObject(Reference));
	}

	bool FToolsetReferenceConverter::JsonDataToProperty(
		const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
		void* OutValue, UObject* Outer)
	{
		FString RefPath;
		if (JsonValue->Type == EJson::Object)
		{
			JsonValue->AsObject()->TryGetStringField(TEXT("refPath"), RefPath);
		}
		else if (JsonValue->Type == EJson::String)
		{
			RefPath = JsonValue->AsString();
		}

		bool IsNull = false;
		if (JsonValue->Type == EJson::Null || StringMeansNull(RefPath))
		{
			RefPath = "None";
			IsNull = true;
		}

		check(!RefPath.IsEmpty());

		if (IsNull)
		{
			// Raise error if null property is a function param and is not either 1) explicitly a
			// FOptionalProperty, or 2) a reference with a default value of null.

			// For FOptionalProperty, check both the property and its owner: Python optional params work by
			// generating a transient value property for the value owned by the optional property.
			const FOptionalProperty* OptionalProperty = CastField<const FOptionalProperty>(Property);
			if (!OptionalProperty)
			{
				OptionalProperty = CastField<const FOptionalProperty>(Property->Owner.ToField());
			}

			if (!OptionalProperty)
			{
				if (const UFunction* OwningFunction = Cast<const UFunction>(Property->Owner.ToUObject()))
				{
					bool HasNullDefault = false;
					FString MetadataKey = FString::Printf(TEXT("CPP_Default_%s"), *GetPropertyDisplayName(Property));
					if (OwningFunction->HasMetaData(*MetadataKey))
					{
						FString DefaultValue = OwningFunction->GetMetaData(*MetadataKey);
						HasNullDefault = StringMeansNull(DefaultValue);  // Reuse existing helper
					}

					if (!HasNullDefault)
					{
						return RejectWith(FString::Printf(TEXT("%s is not valid value for property '%s'"),
							*RefPath, *GetPropertyDisplayName(Property)));
					}
				}
			}
		}
		else
		{
			// Reject bare package paths (e.g. /Game/Foo/Bar) that are missing the
			// Package.Object separator. Paths with a dot are passed through to the
			// type-specific branches below which perform their own validation.
			if (FPackageName::ObjectPathToPathWithinPackage(RefPath).IsEmpty())
			{
				return RejectWith(FString::Printf(TEXT("%s is not a valid object path for property '%s'"),
					*RefPath, *GetPropertyDisplayName(Property)));
			}

			FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);
			const FClassProperty* ClassProperty = CastField<const FClassProperty>(Property);
			const FSoftClassProperty* SoftClassProperty = CastField<const FSoftClassProperty>(Property);

			if (ClassProperty || SoftClassProperty ||
				(ObjectProperty && ObjectProperty->PropertyClass == UClass::StaticClass()))
			{
				const UClass* MetaClass = ClassProperty ? ClassProperty->MetaClass :
										  (SoftClassProperty ? SoftClassProperty->MetaClass : nullptr);
				UObject* Object = FSoftObjectPath(RefPath).ResolveObject();
				if (!Object)
				{
					Object = FSoftObjectPath(RefPath).TryLoad();
				}
				if (!Object || !Object->IsA(UClass::StaticClass()))
				{
					return RejectWith(FString::Printf(TEXT("%s is not valid %s for property '%s'"),
						*RefPath, *UClass::StaticClass()->GetName(), *GetPropertyDisplayName(Property)));
				}
				else if (MetaClass && !Cast<UClass>(Object)->IsChildOf(MetaClass))
				{
					return RejectWith(FString::Printf(TEXT("%s is not valid %s for property '%s'"),
						*RefPath, *MetaClass->GetName(), *GetPropertyDisplayName(Property)));
				}

				// Set property value directly via FSoftObjectPath resolution.
				// ImportText_Direct uses FPropertyHelpers::ReadToken which stops at '[' and ' '
				// characters, truncating paths with bracket-named subobjects.
				if (ClassProperty)
				{
					ClassProperty->SetObjectPropertyValue(OutValue, Cast<UClass>(Object));
				}
				else if (ObjectProperty)
				{
					ObjectProperty->SetObjectPropertyValue(OutValue, Object);
				}
				else
				{
					// FSoftClassProperty — fall through to ImportText_Direct
					FStringOutputDevice ImportError;
					Property->ImportText_Direct(*RefPath, OutValue, nullptr, 0, &ImportError);
					if (!ImportError.IsEmpty())
					{
						return RejectWith(ImportError);
					}
				}
				return true;
			}
			else if (ObjectProperty)
			{
				if (ObjectProperty->HasAnyPropertyFlags(CPF_PersistentInstance))
				{
					// refPath may be a class path (new instance) or a subobject path
					// (round-tripped from get_properties). Try class first, then object.
					UClass* SubClass = FSoftClassPath(RefPath).TryLoadClass<UObject>();
					UObject* ResolvedObj = nullptr;
					if (!SubClass)
					{
						ResolvedObj = FSoftObjectPath(RefPath).ResolveObject();
						if (ResolvedObj)
						{
							SubClass = ResolvedObj->GetClass();
						}
					}
					if (!SubClass || !SubClass->IsChildOf(ObjectProperty->PropertyClass))
					{
						return RejectWith(FString::Printf(TEXT("%s is not a valid subclass of %s for property '%s'"),
							*RefPath, *ObjectProperty->PropertyClass->GetName(),
							*GetPropertyDisplayName(Property)));
					}
					// Back-calculating from GetOffset_ForInternal() is UB for struct member properties
					// (the offset is relative to the struct, not a UObject), so we never do it.
					UObject* ImportOuter = Outer;
					if (ResolvedObj)
					{
						// Subobject path: verify the resolved object is within the outer's hierarchy.
						// Use IsIn() rather than a strict GetOuter() == ImportOuter check so that
						// transitively-owned subobjects (e.g. sub-sub-objects) are also accepted.
						if (!ImportOuter || !ResolvedObj->IsIn(ImportOuter))
						{
							return RejectWith(FString::Printf(
								TEXT("%s is not a subobject of the property owner for instanced property '%s'"),
								*RefPath, *GetPropertyDisplayName(Property)));
						}
						ObjectProperty->SetObjectPropertyValue(OutValue, ResolvedObj);
						return true;
					}
					// Class path -- create a new instance with the correct outer.
					if (!ImportOuter)
					{
						return RejectWith(FString::Printf(
							TEXT("no outer provided for instanced property '%s' — cannot create new object"),
							*GetPropertyDisplayName(Property)));
					}
					UObject* Existing = ObjectProperty->GetObjectPropertyValue(OutValue);
					if (!Existing || Existing->GetClass() != SubClass)
					{
						Existing = NewObject<UObject>(ImportOuter, SubClass);
						ObjectProperty->SetObjectPropertyValue(OutValue, Existing);
					}
					return true;
				}
				else
				{
					UObject* Object = FSoftObjectPath(RefPath).ResolveObject();
					if (!Object)
					{
						Object = FSoftObjectPath(RefPath).TryLoad();
					}
					if (!Object || !Object->IsA(ObjectProperty->PropertyClass))
					{
						return RejectWith(FString::Printf(TEXT("%s is not valid %s for property '%s'"),
							*RefPath, *ObjectProperty->PropertyClass->GetName(),
							*GetPropertyDisplayName(Property)));
					}

					// Set property value directly — bypasses ImportText_Direct's broken
					// tokenizer which truncates paths at '[' bracket characters.
					ObjectProperty->SetObjectPropertyValue(OutValue, Object);
					return true;
				}
			}
			// Validate soft references using Asset Registry
			else if (FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
			{
				FString ErrorMessage;
				if (!ValidateSoftReferenceType(RefPath, SoftObjectProperty->PropertyClass, ErrorMessage))
				{
					return RejectWith(FString::Printf(TEXT("%s '%s'"),
						*ErrorMessage, *GetPropertyDisplayName(Property)));
				}
			}
			else if (FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
			{
				// WeakObjectProperty: validate using Asset Registry (permissive for runtime objects)
				// Weak references typically point to objects that are already loaded
				FString ErrorMessage;
				if (!ValidateSoftReferenceType(RefPath, WeakObjectProperty->PropertyClass, ErrorMessage))
				{
					return RejectWith(FString::Printf(TEXT("%s '%s'"),
						*ErrorMessage, *GetPropertyDisplayName(Property)));
				}
			}
			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				const UScriptStruct* ScriptStruct = StructProperty->Struct;

				if (ScriptStruct == FSoftObjectPath::StaticStruct())
				{
					// FSoftObjectPath: generic soft reference to any UObject
					// No specific class constraint, so we can only check if asset exists
					FAssetRegistryModule& AssetRegistryModule =
						FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
					FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(RefPath));

					// Permissive: only log warning if not found, don't error
					if (!AssetData.IsValid())
					{
						UE_LOGF(LogToolsetRegistry, Warning,
							   "Soft reference path '%ls' not found in Asset Registry for property '%ls'",
							   *RefPath, *GetPropertyDisplayName(Property));
					}
				}
				else if (ScriptStruct == FSoftClassPath::StaticStruct())
				{
					// FSoftClassPath: validate it's actually a class
					FAssetRegistryModule& AssetRegistryModule =
						FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
					FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(RefPath));

					if (AssetData.IsValid())
					{
						UClass* AssetClass = AssetData.GetClass();
						if (AssetClass &&
							!AssetClass->IsChildOf(UClass::StaticClass()) &&
							!AssetClass->IsChildOf(UBlueprint::StaticClass()))
						{
							return RejectWith(FString::Printf(TEXT("%s is not a valid class for property '%s'"),
								*RefPath, *GetPropertyDisplayName(Property)));
						}
					}
				}
			}
		}

		// Fallback for non-object properties (soft refs, struct paths, etc.)
		FStringOutputDevice ImportError;
		Property->ImportText_Direct(*RefPath, OutValue, nullptr, 0, &ImportError);
		if (!ImportError.IsEmpty())
		{
			return RejectWith(ImportError);
		}
		return true;
	}
}
