// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetJsonConverter.h"

#include "ReferenceConverter.generated.h"

/// Represents a reference to a UObject or UClass.
USTRUCT(BlueprintType)
struct FToolsetReference
{
	GENERATED_BODY()
public:
	/// The reference stored as a soft path string.
	UPROPERTY(BlueprintReadWrite, Category = "Toolset")
	FString RefPath;
};

namespace UE::ToolsetRegistry
{
	/// Converts UObject and UClass references.
	class FToolsetReferenceConverter : public FToolsetJsonConverter
	{
	public:
		virtual FString GetName() const  override;

		virtual bool CanConvertProperty(TNotNull<const FProperty*> Property)  override;

		virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(
			TNotNull<const FProperty*> Property)  override;

		virtual TSharedPtr<FJsonValue> PropertyToDefault(
			TNotNull<const FProperty*> Property, const FString& DefaultString)  override;

		virtual TSharedPtr<FJsonValue> PropertyToJsonData(
			TNotNull<FProperty*> Property, const void* Value)  override;

		virtual bool JsonDataToProperty(
			const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
			void* OutValue, UObject* Outer)  override;
	};
}

