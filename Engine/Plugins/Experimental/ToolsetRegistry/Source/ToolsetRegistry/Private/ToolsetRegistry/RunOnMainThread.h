// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Async/Async.h"
#include "Async/Future.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ToolsetRegistry::Internal
{
	// Run a function on the main thread as soon as possible.
	template<typename CallableT>
	inline auto RunOnMainThread(CallableT&& Callable) -> 
		TFuture<decltype(Forward<CallableT>(Callable)())>
	{
		if (IsInGameThread())
		{
			using ReturnT = decltype(Forward<CallableT>(Callable)());
			if constexpr (std::is_void_v<ReturnT>)
			{
				Callable();
				return MakeFulfilledPromise<void>().GetFuture();
			}
			else
			{
				return MakeFulfilledPromise<ReturnT>(Callable()).GetFuture();
			}
		}
		else
		{
			return Async(EAsyncExecution::TaskGraphMainThread, Forward<CallableT>(Callable));
		}
	}
}