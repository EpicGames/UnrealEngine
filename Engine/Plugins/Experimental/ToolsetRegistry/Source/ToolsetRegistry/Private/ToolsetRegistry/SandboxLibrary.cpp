// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/SandboxLibrary.h"

#include "ToolsetRegistry/Module.h"
#include "FileSandboxCoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Data/SandboxMetaData.h"
#include "Types/EBreakBehavior.h"
#include "Types/SandboxedFileChangeInfo.h"
#include "Types/Sandbox/PersistArgs.h"
#include "Types/Sandbox/PersistResult.h"
#include "Types/Sandbox/RevertArgs.h"
#include "Types/Sandbox/RevertResult.h"

namespace
{
	struct FSandboxAccess
	{
		UE::FileSandboxCore::ISandboxManager* Manager = nullptr;
		UE::FileSandboxCore::ISandboxInstance* ActiveInstance = nullptr;
	};

	FSandboxAccess GetSandboxAccess(bool bLogErrorIfNoManager = false)
	{
		using namespace UE::FileSandboxCore;

		if (!IFileSandboxCoreModule::IsAvailable())
		{
			if (bLogErrorIfNoManager)
			{
				UE_LOGF(LogToolsetRegistry, Error, "FileSandbox module is not available");
			}
			return {};
		}

		ISandboxManager* Manager = &IFileSandboxCoreModule::Get().GetSandboxManager();
		return { Manager, Manager->GetActiveSandboxInstance() };
	}
}

namespace UE::ToolsetRegistry
{
	bool FGlobalSandbox::IsActive()
	{
		return GetSandboxAccess().ActiveInstance != nullptr;
	}

	FString FGlobalSandbox::GetActiveName()
	{
		FSandboxAccess Access = GetSandboxAccess();
		return Access.ActiveInstance ? Access.ActiveInstance->GetInitialMetaData().Name : FString();
	}

	bool FGlobalSandbox::Enter(const FString& Name, const FString& Description)
	{
		using namespace UE::FileSandboxCore;

		FSandboxAccess Access = GetSandboxAccess(/*bLogErrorIfNoManager=*/true);
		if (!Access.Manager)
		{
			return false;
		}

		ISandboxManager* Manager = Access.Manager;

		// If already in the requested sandbox, nothing to do.
		if (Access.ActiveInstance)
		{
			if (Access.ActiveInstance->GetInitialMetaData().Name == Name)
			{
				return true;
			}

			// A different sandbox is active — leave it before entering the new one.
			FLeaveSandboxResult LeaveResult = Manager->LeaveSandbox();
			if (LeaveResult.ErrorCode != ELeaveSandboxErrorCode::Success)
			{
				UE_LOGF(LogToolsetRegistry, Error, "Failed to leave active sandbox before entering '%ls'", *Name);
				return false;
			}
		}

		// Try to load an existing sandbox by name.
		FLoadSandboxResult LoadResult = Manager->LoadNamedSandbox(FLoadSandboxByNameArgs(Name));
		if (LoadResult.HasValue())
		{
			UE_LOGF(LogToolsetRegistry, Display, "Resumed existing sandbox: %ls", *Name);
			return true;
		}

		// Create a new sandbox.
		FNewSandboxArgs CreateArgs(Name, Description);
		FNewSandboxResult CreateResult = Manager->CreateNewSandbox(CreateArgs);
		if (CreateResult.HasValue())
		{
			UE_LOGF(LogToolsetRegistry, Display, "Created new sandbox: %ls", *Name);
			return true;
		}

		UE_LOGF(LogToolsetRegistry, Error, "Failed to enter sandbox: %ls", *Name);
		return false;
	}

	bool FGlobalSandbox::Leave()
	{
		using namespace UE::FileSandboxCore;

		FSandboxAccess Access = GetSandboxAccess();
		if (!Access.Manager || !Access.ActiveInstance)
		{
			return true;
		}

		FLeaveSandboxResult Result = Access.Manager->LeaveSandbox();
		return Result.ErrorCode == ELeaveSandboxErrorCode::Success;
	}

	TArray<UE::FileSandboxCore::FSandboxedFileChangeInfo> FGlobalSandbox::GetChanges()
	{
		using namespace UE::FileSandboxCore;

		TArray<FSandboxedFileChangeInfo> Changes;
		ISandboxInstance* Instance = GetSandboxAccess().ActiveInstance;
		if (!Instance)
		{
			return {};
		}

		Instance->EnumerateFileChanges(
			[&Changes](const FSandboxedFileChangeInfo& Info) -> EBreakBehavior
			{
				Changes.Add(Info);
				return EBreakBehavior::Continue;
			},
			EFileEnumerationFlags::All);

		return Changes;
	}

	bool FGlobalSandbox::Persist(const TArray<FString>& Files)
	{
		using namespace UE::FileSandboxCore;

		ISandboxInstance* Instance = GetSandboxAccess().ActiveInstance;
		if (!Instance)
		{
			UE_LOGF(LogToolsetRegistry, Error, "No active sandbox to persist");
			return false;
		}

		if (Files.IsEmpty())
		{
			return Instance->PersistAll();
		}

		FPersistArgs Args;
		Args.Files = Files;
		FPersistResult Result = Instance->PersistSandbox(Args);
		return Result.PersistStatus == EPersistStatus::Success;
	}

	bool FGlobalSandbox::Discard()
	{
		using namespace UE::FileSandboxCore;

		ISandboxInstance* Instance = GetSandboxAccess().ActiveInstance;
		if (!Instance)
		{
			UE_LOGF(LogToolsetRegistry, Error, "No active sandbox to discard");
			return false;
		}

		Instance->RevertAll();
		return true;
	}

	bool FGlobalSandbox::DiscardFiles(const TArray<FString>& Files)
	{
		if (Files.IsEmpty())
		{
			UE_LOGF(LogToolsetRegistry, Error,
				"DiscardFiles called with empty file list; ignoring to avoid discarding all sandbox changes");
			return false;
		}

		using namespace UE::FileSandboxCore;

		ISandboxInstance* Instance = GetSandboxAccess().ActiveInstance;
		if (!Instance)
		{
			UE_LOGF(LogToolsetRegistry, Error, "No active sandbox to discard files from");
			return false;
		}

		FRevertResult Result = Instance->RevertSandbox(FRevertArgs(Files));
		// FRevertResult carries only packages pending hot-reload or purge - it has no
		// error field, so the operation is considered to always succeed once the instance
		// is available. Log the counts for diagnostics.
		UE_LOGF(LogToolsetRegistry, Verbose,
			"RevertSandbox: %d package(s) pending hot-reload, %d package(s) pending purge",
			Result.PackagesPendingHotReload.Num(), Result.PackagesPendingPurge.Num());
		return true;
	}
}
