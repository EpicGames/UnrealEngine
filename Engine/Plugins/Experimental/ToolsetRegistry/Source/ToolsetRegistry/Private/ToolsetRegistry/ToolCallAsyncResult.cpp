// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolCallAsyncResult.h"

#include "Async/UniqueLock.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "UObject/Class.h"

#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/RunOnMainThread.h"
#include "ToolsetRegistry/ToolCallAsyncResultVoid.h"
#include "ToolsetRegistry/ToolsetJson.h"

FString UToolCallAsyncResult::ValuePropertyName = TEXT("Value");

TSharedPtr<FJsonValue> UToolCallAsyncResult::GetValueAsJson() const
{
	if (!bIsComplete || !Error.IsEmpty()) return TSharedPtr<FJsonValue>();

	UClass* Class = GetClass();
	check(Class);
	FProperty* Property = Class->FindPropertyByName(*ValuePropertyName);
	if (!Property)
	{
		UE_LOGF(
			LogToolsetRegistry, Error, "Unable to get Value property from class '%ls'",
			*Class->GetName());
		return TSharedPtr<FJsonValue>();
	}
	// The non-templated FProperty::ContainerVoidPtrToValuePtrInternal() and
	// FProperty::PointerToValuePtr() are not exposed, so manually calculate the offset of
	// the property from the container (UObject) base pointer using the same approach as
	// FProperty::PointerToValuePtr() here.
	const uint8* PropertyValuePointer =
		reinterpret_cast<const uint8*>(this) + Property->GetOffset_ForInternal();
	return UE::ToolsetRegistry::Internal::ToolsetJson::PropertyToJsonData(
		Property, PropertyValuePointer);
}

FString UToolCallAsyncResult::GetValueAsJsonString() const
{
	if (Error.IsEmpty())
	{
		TSharedPtr<FJsonValue> JsonValue = GetValueAsJson();
		if (JsonValue)
		{
			return UE::ToolsetRegistry::Internal::JsonToString(JsonValue.ToSharedRef());
		}
	}
	return TEXT("");
}

bool UToolCallAsyncResult::SetError(const FString& InError)
{
	if (InError.Len() == 0)
	{
		UE_LOGF(
			LogToolsetRegistry, Error,
			"Attempted to complete '%ls' with an empty error!",
			*GetClass()->GetName());
		return false;
	}
	return MaybeBroadcastCompletion(
		[this, CopiedError = InError]() mutable -> void
		{
			this->Error = MoveTemp(CopiedError);
		});
}

bool UToolCallAsyncResult::BroadcastOnCompletedIfComplete()
{
	bool bAlreadyComplete = GetIsComplete();
	if (bAlreadyComplete)
	{
		TStrongObjectPtr<UToolCallAsyncResult> Result(this);
		UE::ToolsetRegistry::Internal::RunOnMainThread(
			[Result = MoveTemp(Result)]() -> void { Result->BroadcastOnCompleted(); });
	}
	return bAlreadyComplete;
}

bool UToolCallAsyncResult::MaybeBroadcastCompletion(
	TFunction<void()>&& SetFinalState)
{
	if (GetIsComplete())
	{
		UE_LOGF(
			LogToolsetRegistry, Error, "Unable to complete '%ls' more than once",
			*GetClass()->GetName());
		return false;
	}
	TStrongObjectPtr<UToolCallAsyncResult> Result(this);
	UE::ToolsetRegistry::Internal::RunOnMainThread(
		[Result = MoveTemp(Result), SetFinalState = MoveTemp(SetFinalState)]() -> void
		{
			SetFinalState();
			Result->BroadcastOnCompleted();
		});
	return true;
}

void UToolCallAsyncResult::SetIsComplete()
{
	UE::TUniqueLock Lock(CompletedLock);
	bIsComplete = true;
}

bool UToolCallAsyncResult::GetIsComplete() const
{
	UE::TUniqueLock Lock(CompletedLock);
	return bIsComplete;
}

void UToolCallAsyncResult::BroadcastOnCompleted()
{
	ensureAlways(IsInGameThread());
	SetIsComplete();
	OnCompleted.Broadcast(this);
	OnCompleted.Clear();
}

TSharedRef<FJsonObject> UToolCallAsyncResult::GetValueJsonSchema(
	TSubclassOf<UToolCallAsyncResult> Class)
{
	using namespace UE::ToolsetRegistry::Internal;
	check(Class);
	FProperty* Property = Class->FindPropertyByName(*ValuePropertyName);
	TSharedPtr<FJsonObject> Schema =
		Property
		? ToolsetJson::PropertyToJsonSchema(Property)
		: UToolCallAsyncResultVoid::GetValueJsonSchema();
	check(Schema.IsValid());
	return Schema.ToSharedRef();
}

UClass* UToolCallAsyncResult::MatchesProperty(TNotNull<const FProperty*> Property)
{
	TObjectPtr<UClass> Class;
	const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);
	if (ObjectProperty)
	{
		Class = ObjectProperty->PropertyClass;
		if (!Class->IsChildOf<UToolCallAsyncResult>()) Class = nullptr;
	}
	return Class;
}