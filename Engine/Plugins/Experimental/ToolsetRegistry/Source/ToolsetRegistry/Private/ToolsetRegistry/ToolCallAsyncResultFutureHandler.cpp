// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolCallAsyncResultFutureHandler.h"

#include "CoreGlobals.h"
#include "Misc/AssertionMacros.h"

#include "ToolsetRegistry/JsonConversion.h"

const FString UToolCallAsyncResultFutureHandler::CanceledError(TEXT("Canceled"));

UToolCallAsyncResultFutureHandler::~UToolCallAsyncResultFutureHandler()
{
	Unsubscribe(true);
}

TFuture<UE::ToolsetRegistry::FJsonValueOrError>
UToolCallAsyncResultFutureHandler::GetValueAsJson()
{
	check(!bRetrievedFuture);
	bRetrievedFuture = true;
	// Hold a strong reference to this object within the future so that it can't be garbage
	// collected until the future is complete.
	TStrongObjectPtr<UToolCallAsyncResultFutureHandler> This(this);
	return ResultPromise.GetFuture().Next(
		[This](UE::ToolsetRegistry::FJsonValueOrError&& Result) ->
			UE::ToolsetRegistry::FJsonValueOrError
		{
			return Result;
		});
}

void UToolCallAsyncResultFutureHandler::Subscribe(UToolCallAsyncResult* ResultIn)
{
	check(IsInGameThread());
	check(ResultIn);
	check(!Result);
	Result = ResultIn;
	Result->OnCompleted.AddUniqueDynamic(this, &ThisClass::OnCompleted);
	Result->BroadcastOnCompletedIfComplete();
}

void UToolCallAsyncResultFutureHandler::Unsubscribe()
{
	Unsubscribe(false);
}

void UToolCallAsyncResultFutureHandler::Unsubscribe(bool bIsDestructing)
{
	check(IsInGameThread());
	// Cancel the promise if it was never fulfilled.
	if (!bOnCompletedHandled)
	{
		ResultPromise.SetValue(MakeError(CanceledError));
		if (!bIsDestructing && Result)
		{
			Result->OnCompleted.RemoveDynamic(this, &ThisClass::OnCompleted);
		}
	}
	bOnCompletedHandled = true;
	Result = nullptr;
}

void UToolCallAsyncResultFutureHandler::OnCompleted(UToolCallAsyncResult* ResultIn)
{
	check(IsInGameThread());
	check(Result == ResultIn);
	check(Result->bIsComplete);
	check(!bOnCompletedHandled);
	const FString& Error = Result->Error;
	if (Error.IsEmpty())
	{
		ResultPromise.SetValue(MakeValue(Result->GetValueAsJson()));
	}
	else
	{
		ResultPromise.SetValue(MakeError(Error));
	}
	bOnCompletedHandled = true;
	Unsubscribe();
}

TStrongObjectPtr<UToolCallAsyncResultFutureHandler>
UToolCallAsyncResultFutureHandler::Create(TObjectPtr<UToolCallAsyncResult> Result)
{
	UToolCallAsyncResultFutureHandler* OnCompletedHandler =
		NewObject<UToolCallAsyncResultFutureHandler>();
	OnCompletedHandler->Subscribe(Result);
	return TStrongObjectPtr<UToolCallAsyncResultFutureHandler>(OnCompletedHandler);
}

