// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolCallAsyncResultVoid.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Templates/SharedPointer.h"

#include "ToolsetRegistry/JsonSchema.h"

TSharedPtr<FJsonValue> UToolCallAsyncResultVoid::GetValueAsJson() const
{
	return bIsComplete ? MakeShared<FJsonValueNull>() : TSharedPtr<FJsonValue>();
}

TSharedRef<FJsonObject> UToolCallAsyncResultVoid::GetValueJsonSchema()
{
	return UE::ToolsetRegistry::Internal::CreateJsonSchema(TEXT("Always null"), EJson::Null);
}