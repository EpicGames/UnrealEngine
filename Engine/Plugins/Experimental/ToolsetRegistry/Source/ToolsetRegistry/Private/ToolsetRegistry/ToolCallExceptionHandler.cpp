// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolCallExceptionHandler.h"

#include "Blueprint/BlueprintExceptionInfo.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Stack.h"

#include "ToolsetRegistry/DelegateHandle.h"
#include "ToolsetRegistry/ToolCallExceptionHandlerInternal.h"

namespace UE::ToolsetRegistry
{
	FToolCallExceptionHandler::FToolCallExceptionHandler()
		: ScriptExceptionHandle(
			FDelegateHandleRaii::Create(
				FBlueprintCoreDelegates::OnScriptException,
				FBlueprintCoreDelegates::OnScriptException.AddRaw(
					this, &FToolCallExceptionHandler::OnScriptException)))
	{}

	void FToolCallExceptionHandler::OnScriptException(
			const UObject* ActiveObject, const FFrame& StackFrame, const FBlueprintExceptionInfo& Info)
	{
		TOptional<FString> ErrorMessage = GetBlueprintExceptionMessage(Info, StackFrame);
		if (ErrorMessage.IsSet())
		{
			AddToExceptionMessage(ErrorMessage.GetValue());
		}
	}

	FString FToolCallExceptionHandler::GetBackupExceptionMessage(
		const FString& ExceptionTypeName, const FFrame& StackFrame)
	{
		// Generate as useful a message as we can if the exception didn't provide one.
		FString FallbackExceptionMessage;
		if (StackFrame.PreviousTrackingFrame &&
			StackFrame.PreviousTrackingFrame->Object &&
			StackFrame.PreviousTrackingFrame->Node)
		{
			FallbackExceptionMessage = FString::Printf(
				TEXT("%s in %s.%s"), *ExceptionTypeName,
				*StackFrame.PreviousTrackingFrame->Object->GetName(),
				*StackFrame.PreviousTrackingFrame->Node->GetName());
		}
		else if (StackFrame.Object && StackFrame.Node)
		{
			FallbackExceptionMessage = FString::Printf(
				TEXT("%s in %s.%s"), *ExceptionTypeName,
				*StackFrame.Object->GetName(),
				*StackFrame.Node->GetName());
		}
		else
		{
			FallbackExceptionMessage = ExceptionTypeName;
		}
		return FallbackExceptionMessage;
	}

	TOptional<FString> FToolCallExceptionHandler::GetBlueprintExceptionMessage(
		const FBlueprintExceptionInfo& Info, const FFrame& StackFrame)
	{
		const FText& Description = Info.GetDescription();
		if (!Description.IsEmpty()) return Description.ToString();

		EBlueprintExceptionType::Type ExceptionType = Info.GetType();
		const FString* ErrorString =
			UE::ToolsetRegistry::Internal::HandledBlueprintExceptionTypeToErrorStrings.Find(
				ExceptionType);
		if (ErrorString) return GetBackupExceptionMessage(*ErrorString, StackFrame);

		checkfSlow(
			UE::ToolsetRegistry::Internal::IgnoredBlueprintExceptionTypes.Contains(ExceptionType),
			TEXT("Unhandled exception type %d"), ExceptionType);
		return TOptional<FString>();
	}

	void FToolCallExceptionHandler::AddToExceptionMessage(const FString& NewMessage)
	{
		if (NewMessage.IsEmpty())
		{
			return;
		}
		else if (ScriptException.IsEmpty())
		{
			ScriptException = NewMessage;
		}
		else
		{
			ScriptException += FString::Printf(TEXT("\n%s"), *NewMessage);
		}
	}

	void FToolCallExceptionHandler::CaptureErrorsIn(TFunction<void()>&& Func)
	{
		UE::ToolsetRegistry::Internal::CallWithinBlueprintScript(MoveTemp(Func));
	}
}

