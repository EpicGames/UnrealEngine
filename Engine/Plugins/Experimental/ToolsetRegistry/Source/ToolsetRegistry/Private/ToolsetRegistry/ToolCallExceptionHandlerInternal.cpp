// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolCallExceptionHandlerInternal.h"

#include "Containers/UnrealString.h"
#include "Templates/UnrealTemplate.h"

#include "ToolsetRegistry/ToolCallExceptionHandlerTestScope.h"

namespace UE::ToolsetRegistry::Internal
{
	const TMap<EBlueprintExceptionType::Type, FString>
		HandledBlueprintExceptionTypeToErrorStrings =
	{
		{EBlueprintExceptionType::AccessViolation, TEXT("AccessViolation")},
		{EBlueprintExceptionType::InfiniteLoop, TEXT("InfiniteLoop")},
		{EBlueprintExceptionType::NonFatalError, TEXT("NonFatalError")},
		{EBlueprintExceptionType::FatalError, TEXT("FatalError")},
		{EBlueprintExceptionType::AbortExecution, TEXT("AbortExecution")},
		{EBlueprintExceptionType::UserRaisedError, TEXT("UserRaisedError")},
	};
	
	const TSet<EBlueprintExceptionType::Type> IgnoredBlueprintExceptionTypes =
	{
		EBlueprintExceptionType::Breakpoint,
		EBlueprintExceptionType::Tracepoint,
		EBlueprintExceptionType::WireTracepoint,
	};

	void CallWithinBlueprintScript(TFunction<void()>&& Func)
	{
		NewObject<UToolCallExceptionHandlerTestScope>()->CallFunction(MoveTemp(Func));
	}
}