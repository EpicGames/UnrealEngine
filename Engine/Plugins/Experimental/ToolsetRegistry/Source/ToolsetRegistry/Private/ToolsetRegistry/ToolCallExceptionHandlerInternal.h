// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Blueprint/BlueprintExceptionInfo.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Templates/Function.h"

namespace UE::ToolsetRegistry::Internal
{
	// Map of FToolCallExceptionHandler handled blueprint exception types to error strings.
	extern const TMap<EBlueprintExceptionType::Type, FString>
		HandledBlueprintExceptionTypeToErrorStrings;
	// Set of FToolCallExceptionHandler ignored blueprint exception types to error strings.
	extern const TSet<EBlueprintExceptionType::Type> IgnoredBlueprintExceptionTypes;

	// Call a function within the scope of a blueprint script.
	void CallWithinBlueprintScript(TFunction<void()>&& Func);
}