// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolCallExceptionHandlerTestScope.h"

#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

void UToolCallExceptionHandlerTestScope::CallFunction(TFunction<void()>&& Func)
{
	FunctionToCall = MoveTemp(Func);

	UFunction* CallFunctionInternalObject =
		FindFunctionChecked(FName(TEXT("CallFunctionInternal")));
	ProcessEvent(CallFunctionInternalObject, nullptr);
	
	FunctionToCall.Reset();
}

void UToolCallExceptionHandlerTestScope::CallFunctionInternal()
{
	if (FunctionToCall.IsSet()) FunctionToCall();
}
