// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Templates/Function.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "ToolCallExceptionHandlerTestScope.generated.h"


// Simplifies testing Blueprint toolsets that raise errors caught by FToolCallExceptionHandler.
// NOTE: This is tested by ToolCallExceptionHandlerTest.cpp.
UCLASS(BlueprintType, HideDropdown)
class UToolCallExceptionHandlerTestScope : public UObject
{
	GENERATED_BODY()

public:
	// Call the specified function within a blueprint execution context.
	void CallFunction(TFunction<void()>&& Func);

protected:
	// Calls FunctionToCall if set.
	UFUNCTION()
	void CallFunctionInternal();

private:
	TFunction<void()> FunctionToCall;
};