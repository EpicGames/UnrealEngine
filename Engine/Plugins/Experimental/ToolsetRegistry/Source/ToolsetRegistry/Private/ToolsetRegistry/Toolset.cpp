// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/Toolset.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Internationalization/Regex.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/NamePatternFilter.h"

namespace UE::ToolsetRegistry
{
	static TOptional<FString> TryGetToolName(const TSharedPtr<FJsonValue>& ToolValue)
	{
		const TSharedPtr<FJsonObject>* ToolObject;
		FString Name;
		if (ToolValue->TryGetObject(ToolObject) &&
			(*ToolObject)->TryGetStringField(TEXT("name"), Name))
		{
			return Name;
		}
		return {};
	}

	FToolset::FToolset() = default;

	FToolset::~FToolset() = default;

	TFuture<TValueOrError<FString, FString>> FToolset::ExecuteTool(
		const FString& ToolName, const FString& JsonInput)
	{
		if (!IsEnabled())
		{
			return MakeFulfilledPromise<TValueOrError<FString, FString>>(
				MakeError(FString(TEXT("Toolset not found")))).GetFuture();
		}
		const FString FullToolName = GetToolsetName() + TEXT(".") + ToolName;
		if (!IsToolEnabled(FullToolName))
		{
			return MakeFulfilledPromise<TValueOrError<FString, FString>>(
				MakeError(FString(TEXT("Tool not found")))).GetFuture();
		}
		return ExecuteToolInternal(ToolName, JsonInput);
	}

	bool FToolset::IsEnabled() const
	{
		return bEnabled;
	}

	void FToolset::SetEnabled(bool bInEnabled)
	{
		bEnabled = bInEnabled;
	}

	void FToolset::SetNameFilters(
		const TArray<FString>& BlockPatterns, const TArray<FString>& AllowPatterns)
	{
		// Compile patterns once for use at both toolset and tool level.
		TArray<FRegexPattern> CompiledBlock = Internal::CompilePatterns(BlockPatterns);
		TArray<FRegexPattern> CompiledAllow = Internal::CompilePatterns(AllowPatterns);

		// Apply to the toolset itself.
		SetEnabled(Internal::IsNameAllowed(GetToolsetName(), CompiledBlock, CompiledAllow));

		// Apply to individual tools.
		DisabledTools.Empty();
		if (BlockPatterns.IsEmpty() && AllowPatterns.IsEmpty())
		{
			return;
		}

		TArray<FString> ToolNames = ListToolNames();
		for (const FString& ToolName : ToolNames)
		{
			if (!Internal::IsNameAllowed(ToolName, CompiledBlock, CompiledAllow))
			{
				DisabledTools.Add(ToolName);
			}
		}
		if (!ToolNames.IsEmpty() && DisabledTools.Num() == ToolNames.Num())
		{
			SetEnabled(false);
		}
	}

	bool FToolset::IsToolEnabled(const FString& FullToolName) const
	{
		return IsEnabled() && !DisabledTools.Contains(FullToolName);
	}

	TArray<FString> FToolset::ListToolNames() const
	{
		TSharedPtr<FJsonObject> Object = Internal::JsonStringToJsonObject(GetJsonSchemaInternal());
		const TArray<TSharedPtr<FJsonValue>>* Tools = nullptr;
		if (!Object.IsValid() || !Object->TryGetArrayField(TEXT("tools"), Tools))
		{
			return {};
		}

		TArray<FString> ToolNames;
		for (const TSharedPtr<FJsonValue>& ToolValue : *Tools)
		{
			if (TOptional<FString> Name = TryGetToolName(ToolValue))
			{
				ToolNames.Add(Name.GetValue());
			}
		}
		return ToolNames;
	}

	FString FToolset::GetJsonSchema() const
	{
		if (!IsEnabled())
		{
			return FString();
		}

		FString Schema = GetJsonSchemaInternal();

		if (DisabledTools.IsEmpty())
		{
			return Schema;
		}

		TSharedPtr<FJsonObject> Object = Internal::JsonStringToJsonObject(Schema);
		const TArray<TSharedPtr<FJsonValue>>* Tools = nullptr;
		if (!Object.IsValid() || !Object->TryGetArrayField(TEXT("tools"), Tools))
		{
			return Schema;
		}

		TArray<TSharedPtr<FJsonValue>> FilteredTools;
		for (const TSharedPtr<FJsonValue>& ToolValue : *Tools)
		{
			TOptional<FString> ToolName = TryGetToolName(ToolValue);
			if (!ToolName.IsSet() || IsToolEnabled(ToolName.GetValue()))
			{
				FilteredTools.Add(ToolValue);
			}
		}

		if (FilteredTools.Num() == Tools->Num())
		{
			return Schema;
		}

		if (FilteredTools.IsEmpty())
		{
			return FString();
		}

		Object->SetArrayField(TEXT("tools"), FilteredTools);
		return Internal::JsonToString(Object.ToSharedRef());
	}

}  // namespace UE::ToolsetRegistry
