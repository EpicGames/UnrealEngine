// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolsetDefinition.h"
#include "ToolsetRegistry/Module.h"

TValueOrError<bool, FString> 
UToolsetDefinition::IsFunctionAICallable(const TObjectPtr<const UFunction>& Function)
{
	check(Function);

	static const FName AIIgnoreName(TEXT("AIIgnore"));
	if (Function->HasMetaData(AIIgnoreName))
	{
		return MakeValue<bool>(false);
	}

	if (!Function->HasAllFunctionFlags(FUNC_Static))
	{
		return MakeError<FString>(TEXT("is not static"));
	}

	static const FName AICallableName(TEXT("AICallable"));
	if (!Function->HasMetaData(AICallableName))
	{
		return MakeError<FString>(TEXT("is not AICallable"));
	}
	return MakeValue<bool>(true);
}
