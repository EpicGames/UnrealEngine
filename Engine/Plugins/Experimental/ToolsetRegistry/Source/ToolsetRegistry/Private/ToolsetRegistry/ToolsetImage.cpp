// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolsetImage.h"

#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Misc/Base64.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolsetImage)

void EncodeImage(const TSharedPtr<IImageWrapper>& ImageWrapper, FString& OutData)
{
	const TArray64<uint8>& CompressedData = ImageWrapper->GetCompressed();
	OutData = FBase64::Encode(CompressedData.GetData(), CompressedData.Num());
}

bool FToolsetImage::SetFromBitmap(const TArray<FColor>& Bitmap, FIntPoint Dimensions, ERGBFormat Format)
{
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>("ImageWrapper");
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

	if (ImageWrapper.IsValid() &&
		ImageWrapper->SetRaw(Bitmap.GetData(), Bitmap.Num() * sizeof(FColor), Dimensions.X, Dimensions.Y, Format, 8))
	{
		EncodeImage(ImageWrapper, Data);
		MimeType = TEXT("image/png");
		return true;
	}

	return false;
}

bool FToolsetImage::SetFromFile(const FString& Path)
{
	TArray<uint8> ImageData;
	if (!FFileHelper::LoadFileToArray(ImageData, *Path))
	{
		return false;
	}

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>("ImageWrapper");

	EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(ImageData.GetData(), ImageData.Num());
	if (ImageFormat != EImageFormat::PNG)
	{
		return false;
	}

	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);
	if (!ImageWrapper.IsValid() || !ImageWrapper->SetCompressed(ImageData.GetData(), ImageData.Num()))
	{
		return false;
	}

	EncodeImage(ImageWrapper, Data);
	MimeType = TEXT("image/png");
	return true;
}
