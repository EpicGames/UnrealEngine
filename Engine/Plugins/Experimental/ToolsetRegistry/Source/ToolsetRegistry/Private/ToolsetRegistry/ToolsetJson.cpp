// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetJson.h"

#include "Editor.h"
#include "JsonGlobals.h"
#include "JsonObjectConverter.h"
#include "JsonObjectStructInterface.h"
#include "JsonSchema/JsonSchemaGenerator.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Modules/ModuleManager.h"
#include "Misc/AssertionMacros.h"

#include "ToolsetRegistry/PropertyAccessors.h"
#include "ToolsetRegistry/ToolCallAsyncResult.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"

namespace
{
	// Use RequiredModules::EnsureLoaded() to lazily load required modules.
	class RequiredModules
	{
	private:
		RequiredModules()
		{
			FModuleManager::Get().LoadModuleChecked(TEXT("JsonUtilitiesEditor"));
		}

	public:
		static void EnsureLoaded()
		{
			// Construct a singleton to ensure required modules are loaded for methods
			// in this class.
			static RequiredModules Load;
		}
	};

	EPropertyFlags GetCheckFlags(bool UserVisiblePropertiesOnly)
	{
		EPropertyFlags CheckFlags = CPF_None;
		if (UserVisiblePropertiesOnly)
		{
			CheckFlags = CPF_BlueprintVisible | CPF_BlueprintAssignable;
			if (GEditor)
			{
				CheckFlags |= CPF_Edit;
			}
		}
		return CheckFlags;
	}
}

namespace UE::ToolsetRegistry::Internal::ToolsetJson
{

// This method generates customizes the default JSON schema generation logic in a way that is easier for an
// LLM to use reliably. The primary motivation for this method is converting UObject* and UClass* from
// "string" types to a distinct reference type.
bool CustomJsonSchema(
	const FProperty* Property, const FString& ParameterDefaultString, const TSharedRef<FJsonObject>& OutputSchema)
{
	RequiredModules::EnsureLoaded();

	if (UClass* Class = UToolCallAsyncResult::MatchesProperty(Property))
	{
		FJsonObject::Duplicate(UToolCallAsyncResult::GetValueJsonSchema(Class), OutputSchema);
		return true;
	}

	auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get(TEXT("CustomJsonSchema"));
	if (ToolsetRegistrySubsystem.HasValue())
	{
		auto Converter = ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.GetConverterForProperty(Property);
		if (Converter)
		{
			if (ParameterDefaultString.IsEmpty())
			{
				TSharedPtr<FJsonObject> PropertyJson = Converter->PropertyToJsonSchema(Property);
				FJsonObject::Duplicate(PropertyJson, OutputSchema);
			}
			else
			{
				OutputSchema->SetField(
					TEXT("default"), Converter->PropertyToDefault(Property, ParameterDefaultString));
			}
			return true;
		}
	}

	return false;
}

// Customizes the default JSON data import logic to match the custom schema and data export
// logic in this file. When one of our registered converters explicitly rejects the JSON input
// (raises a script error and returns false), CustomImport sets bRejected = true but still
// returns true — because the FJsonObjectConverter CustomImportCallback contract is
// "return true = I handled it; false = fall through to default behavior". Returning false on
// validation failure would cause JsonObjectConverter to silently create a new default instance
// (for instanced FObjectProperty), defeating the validation. The bRejected flag lets the outer
// wrapper detect this and report failure. Each call to JsonDataToProperty / JsonDataToStruct
// has its own bRejected local, so nested invocations don't poison each other.
bool CustomImport(
	const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* Value,
	UObject* Outer, bool& bRejected)
{
	ensureAlways(JsonValue.IsValid());
	RequiredModules::EnsureLoaded();

	auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get(TEXT("CustomJsonImport"));
	if (ToolsetRegistrySubsystem.HasValue())
	{
		auto Converter = ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.GetConverterForProperty(Property);
		if (Converter)
		{
			if (!Converter->JsonDataToProperty(JsonValue, Property, Value, Outer))
			{
				bRejected = true;
			}
			return true;
		}
	}
	return false;
}

// This method customizes the default JSON data export behavior to align with the
// custom schema generated by CustomJsonSchema().
TSharedPtr<FJsonValue> CustomExport(FProperty* Property, const void* Value)
{
	RequiredModules::EnsureLoaded();

	if (auto Result = PropertyValueAsObject<UToolCallAsyncResult>(Property, Value);
		Result.IsSet())
	{
		return *Result ? (*Result)->GetValueAsJson() : nullptr;
	}

	auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get(TEXT("CustomJsonExport"));
	if (ToolsetRegistrySubsystem.HasValue())
	{
		auto Converter = ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.GetConverterForProperty(Property);
		if (Converter)
		{
			return Converter->PropertyToJsonData(Property, Value);
		}
	}
	
	return nullptr;
}

bool HasCustomImporter(TNotNull<const FProperty*> Property)
{
	RequiredModules::EnsureLoaded();
	auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get(TEXT("HasCustomImporter"));
	return ToolsetRegistrySubsystem.HasValue() &&
		ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.GetConverterForProperty(Property) != nullptr;
}

TSharedPtr<FJsonObject> PropertyToJsonSchema(TNotNull<const FProperty*> Property)
{
	RequiredModules::EnsureLoaded();
	FJsonSchemaPropertyFilter::CustomCallback CustomCallback;
	CustomCallback.BindStatic(CustomJsonSchema);
	FJsonSchemaPropertyFilter PropertyFilter;
	PropertyFilter.CustomCb = &CustomCallback;
	return FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Property, PropertyFilter);
}

TSharedPtr<FJsonValue> PropertyToJsonData(
	TNotNull<FProperty*> Property, TNotNull<const void*> Value)
{
	RequiredModules::EnsureLoaded();
	FJsonObjectConverter::CustomExportCallback CustomExportCallback;
	CustomExportCallback.BindStatic(CustomExport);
	return FJsonObjectConverter::UPropertyToJsonValue(Property, Value, 0, 0, &CustomExportCallback);
}

bool JsonDataToProperty(
	const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, TNotNull<void*> OutValue,
	UObject* Outer)
{
	RequiredModules::EnsureLoaded();
	bool bRejected = false;
	FJsonObjectConverter::CustomImportCallback CustomImportCallback;
	// TODO: this will always pass our Outer as the Outer for the CustomImport, even when technically
    // this should switch to an instanced subobject when recursing inside objects.
    // This would require updating the actual FJsonObjectConverter::CustomImportCallback to have the
    // current Outer as a parameter, which it could populate as it recurses.
	CustomImportCallback.BindLambda(
		[Outer, &bRejected](const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* Value)
		{
			return CustomImport(JsonValue, Property, Value, Outer, bRejected);
		});
	// FJsonObjectConverter logs a generic "Unable to import" error on LogJson when any property
	// fails to convert. Our converters already raise specific, actionable script errors via
	// RaiseScriptError before signalling rejection; the generic log noise just clutters output.
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogJson, ELogVerbosity::Fatal);
	const bool bOk = FJsonObjectConverter::JsonValueToUProperty(
		JsonValue, Property, OutValue, 0, 0, false, nullptr, &CustomImportCallback);
	return bOk && !bRejected;
}

TSharedPtr<FJsonObject> StructToJsonSchema(
	TNotNull<const UStruct*> StructDefinition, bool UserVisiblePropertiesOnly)
{
	RequiredModules::EnsureLoaded();
	FJsonSchemaPropertyFilter::CustomCallback CustomCallback;
	CustomCallback.BindStatic(CustomJsonSchema);
	FJsonSchemaPropertyFilter PropertyFilter;
	PropertyFilter.CustomCb = &CustomCallback;
	PropertyFilter.CheckFlags = GetCheckFlags(UserVisiblePropertiesOnly);
	PropertyFilter.SkipFlags = CPF_Deprecated;
	return FJsonSchemaGenerator::UStructToJsonSchemaObject(StructDefinition, PropertyFilter);
}

TSharedPtr<FJsonObject> StructToJsonData(
	TNotNull<const UStruct*> StructDefinition, TNotNull<const void*> Struct, bool UserVisiblePropertiesOnly)
{
	RequiredModules::EnsureLoaded();
	FJsonObjectConverter::CustomExportCallback CustomExportCallback;
	CustomExportCallback.BindStatic(CustomExport);
	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	EPropertyFlags CheckFlags = GetCheckFlags(UserVisiblePropertiesOnly);
	EPropertyFlags SkipFlags = CPF_Deprecated;
	if (FJsonObjectConverter::UStructToJsonObject(
		StructDefinition, Struct, JsonObject.ToSharedRef(), CheckFlags, SkipFlags, &CustomExportCallback))
	{
		return JsonObject;
	}
	return nullptr;
}

bool JsonDataToStruct(
	const TSharedRef<FJsonObject>& JsonObject, TNotNull<const UStruct*> StructDefinition, TNotNull<void*> OutStruct)
{
	RequiredModules::EnsureLoaded();
	bool bRejected = false;
	FJsonObjectConverter::CustomImportCallback CustomImportCallback;
	CustomImportCallback.BindLambda(
		[&bRejected](const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* Value)
		{
			return CustomImport(JsonValue, Property, Value, nullptr, bRejected);
		});
	// See note in JsonDataToProperty.
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogJson, ELogVerbosity::Fatal);
	const bool bOk = FJsonObjectConverter::JsonObjectToUStruct(
		JsonObject, StructDefinition, OutStruct, 0, 0, false, nullptr, &CustomImportCallback);
	return bOk && !bRejected;
}

}  // namespace UE::ToolsetRegistry::Internal::ToolsetJson
