// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

class FJsonObject;
class FJsonValue;


/// Contains ToolsetRegistry specific wrappers for JSON schema and data generation.
/// These wrappers perform custom transformations on specific property types, like
/// converting UObject* and UClass* to FToolsetReference (vs the default in
/// JsonUtilities which is a string).
namespace UE::ToolsetRegistry::Internal::ToolsetJson
{
	/*
	 * Converts an FProperty to JSON schema.
	 * @param Property The FProperty to convert.
	 * @return JSON schema that represents the FProperty.
	 */
	TSharedPtr<FJsonObject> PropertyToJsonSchema(TNotNull<const FProperty*> Property);
	/*
	 * Gets an FProperty on an instance to JSON data.
	 * @param Property The FProperty to convert.
	 * @param Value The object or struct to get the value from. 
	 * @return JSON data that represents the value of the FProperty.
	 */
	TSharedPtr<FJsonValue> PropertyToJsonData(
		TNotNull<FProperty*> Property, TNotNull<const void*> Value);
	/*
	 * Set an FProperty on an instance from JSON data.
	 * @param JsonValue The JSON data to convert.
	 * @param Property The FProperty to apply the data to.
	 * @param OutValue The object or struct to modify.
	 * @return True if the data was converted successfully.
	 */
	bool JsonDataToProperty(
		const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, TNotNull<void*> OutValue,
		UObject* Outer = nullptr);

	// Returns true if the ToolsetJson converter registry would handle this property,
	// meaning callers must not bypass it by recursing field-by-field.
	bool HasCustomImporter(TNotNull<const FProperty*> Property);

	/*
	 * Converts a UStruct to JSON schema.
	 * @param StructDefinition The UStruct type to convert.
	 * @param UserVisiblePropertiesOnly Whether to export all properties or only the ones that appear in editor.
	 * @return JSON schema that represents the UStruct.
	 */
	TSharedPtr<FJsonObject> StructToJsonSchema(
		TNotNull<const UStruct*> StructDefinition, bool UserVisiblePropertiesOnly = false);
	/*
	 * Converts a UStruct instance to JSON data.
	 * @param StructDefinition The UStruct type to convert.
	 * @param Struct The UStruct instance to convert.
	 * @param UserVisiblePropertiesOnly Whether to export all properties or only the ones that appear in editor.
	 * @return JSON data that represents the UStruct instance.
	 */
	TSharedPtr<FJsonObject> StructToJsonData(
		TNotNull<const UStruct*> StructDefinition, TNotNull<const void*> Struct,
		bool UserVisiblePropertiesOnly = false);
	/*
	 * Updates a UStruct instance from JSON data.
	 * @param JsonObject The JSON data to convert.
	 * @param StructDefinition The UStruct type.
	 * @param OutStruct The UStruct instance to modify.
	 * @return True if the data was applied successfully.
	 */
	bool JsonDataToStruct(
		const TSharedRef<FJsonObject>& JsonObject, TNotNull<const UStruct*> StructDefinition,
		TNotNull<void*> OutStruct);
}  // namespace UE::ToolsetRegistry::Internal::ToolsetJson
