// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolsetJsonConverter.h"
#include "ToolsetJson.h"

namespace UE::ToolsetRegistry
{
	FToolsetJsonConverter::FToolsetJsonConverter() = default;

	FToolsetJsonConverter::~FToolsetJsonConverter() = default;

	TSharedPtr<FJsonObject> FToolsetJsonConverter::ToolsetPropertyToJsonSchema(
		TNotNull<const FProperty*> Property)
	{
		return Internal::ToolsetJson::PropertyToJsonSchema(Property);
	}

	TSharedPtr<FJsonValue> FToolsetJsonConverter::ToolsetPropertyToJsonData(
		TNotNull<FProperty*> Property, TNotNull<const void*> Value)
	{
		return Internal::ToolsetJson::PropertyToJsonData(Property, Value);
	}

	bool FToolsetJsonConverter::ToolsetJsonDataToProperty(
		const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, TNotNull<void*> OutValue)
	{
		return Internal::ToolsetJson::JsonDataToProperty(JsonValue, Property, OutValue);
	}

	
	TSharedPtr<FJsonObject> FToolsetJsonConverter::ToolsetStructToJsonSchema(
		TNotNull<const UStruct*> StructDefinition, bool UserVisiblePropertiesOnly)
	{
		return Internal::ToolsetJson::StructToJsonSchema(StructDefinition, UserVisiblePropertiesOnly);
	}

	TSharedPtr<FJsonObject> FToolsetJsonConverter::ToolsetStructToJsonData(
		TNotNull<const UStruct*> StructDefinition, TNotNull<const void*> Struct,
		bool UserVisiblePropertiesOnly)
	{
		return Internal::ToolsetJson::StructToJsonData(StructDefinition, Struct, UserVisiblePropertiesOnly);
	}

	bool FToolsetJsonConverter::ToolsetJsonDataToStruct(
		const TSharedRef<FJsonObject>& JsonObject, TNotNull<const UStruct*> StructDefinition,
		TNotNull<void*> OutStruct)
	{
		return Internal::ToolsetJson::JsonDataToStruct(JsonObject, StructDefinition, OutStruct);
	}

}  // namespace UE::ToolsetRegistry