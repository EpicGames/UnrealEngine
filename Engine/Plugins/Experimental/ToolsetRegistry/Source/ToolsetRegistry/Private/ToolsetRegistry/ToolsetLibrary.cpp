// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolsetLibrary.h"

#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "Misc/AssertionMacros.h"
#include "UObject/PropertyAccessUtil.h"
#include "Kismet/KismetSystemLibrary.h"
#include "UnrealClient.h"

#include "Editor.h"
#include "Editor/Transactor.h"

#include "ToolsetRegistry/JsonConversion.h"
#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/NamePatternFilter.h"
#include "ToolsetRegistry/ToolsetJson.h"
#include "ToolsetRegistry/ToolsetLibraryImpl.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolsetLibrary)

FString UToolsetLibrary::ListStructProperties(
	const UStruct* Struct, bool bUserVisiblePropertiesOnly)
{
	if (!Struct)
	{
		return FString();
	}
	FString JsonProperties;
	TSharedPtr<FJsonObject> ClassSchema =
		UE::ToolsetRegistry::Internal::ToolsetJson::StructToJsonSchema(Struct, bUserVisiblePropertiesOnly);
	const TSharedPtr<FJsonObject>* ClassProperties = nullptr;
	if (ClassSchema->TryGetObjectField(TEXT("properties"), ClassProperties))
	{
		check(ClassProperties);
		JsonProperties = UE::ToolsetRegistry::Internal::JsonToString(
			ClassProperties->ToSharedRef());
	}
	return JsonProperties;
}

FString UToolsetLibrary::GetObjectProperties(const UObject* Object, const TArray<FName>& PropertyNames)
{
	if (!Object)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Null object provided"));
		return FString();
	}
	TSharedPtr<FJsonObject> PropertyData = MakeShared<FJsonObject>();
	TArray<FString> InaccessibleProperties;
	for (const FName& PropertyName : PropertyNames)
	{
		FProperty* Property = PropertyAccessUtil::FindPropertyByName(PropertyName, Object->GetClass());
		if (!Property)
		{
			InaccessibleProperties.Add(PropertyName.ToString());
			continue;
		}
		const EPropertyAccessResultFlags AccessResult = PropertyAccessUtil::CanGetPropertyValue(Property);
		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::PermissionDenied))
		{
			InaccessibleProperties.Add(PropertyName.ToString());
			continue;
		}
		PropertyData->SetField(
			PropertyName.ToString(),
			UE::ToolsetRegistry::Internal::ToolsetJson::PropertyToJsonData(
				Property, Property->ContainerPtrToValuePtr<void>(Object)));
	}
	if (!InaccessibleProperties.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("GetObjectProperties on '%s' (%s): the following properties could not be read: %s"),
			*Object->GetPathName(), *Object->GetClass()->GetName(),
			*FString::Join(InaccessibleProperties, TEXT(", "))));
	}
	return UE::ToolsetRegistry::Internal::JsonToString(PropertyData.ToSharedRef());
}

bool UToolsetLibrary::SetObjectProperties(UObject* Object, const FString& PropertiesJson,
	EBypassContainerCheck BypassContainerCheck)
{
	TArray<FName> Ignored;
	return SetObjectProperties(Object, PropertiesJson, Ignored, BypassContainerCheck);
}

bool UToolsetLibrary::SetObjectProperties(UObject* Object, const FString& PropertiesJson,
	TArray<FName>& OutSetPropertyNames, EBypassContainerCheck BypassContainerCheck)
{
	using namespace UE::ToolsetRegistry::Internal;

	if (!Object)
	{
		return false;
	}

	TSharedPtr<FJsonObject> PropertyData =
		JsonStringToJsonObject(PropertiesJson);
	if (!PropertyData.IsValid())
	{
		return false;
	}

	const UClass* ObjectClass = Object->GetClass();

	// Block list: consult UToolsetRegistrySettings before any write. Compile
	// the configured patterns once per call; on an empty list this is a no-op.
	const UToolsetRegistrySettings* BlockListSettings = GetDefault<UToolsetRegistrySettings>();
	const TArray<FRegexPattern> ClassBlockPatterns =
		CompilePatterns(BlockListSettings->SetObjectPropertiesBlockedClasses);
	const TArray<FRegexPattern> PropertyBlockPatterns =
		CompilePatterns(BlockListSettings->SetObjectPropertiesBlockedProperties);

	if (!ClassBlockPatterns.IsEmpty())
	{
		for (const UClass* Class = ObjectClass; Class; Class = Class->GetSuperClass())
		{
			const FString ClassName = Class->GetName();
			for (const FRegexPattern& Pattern : ClassBlockPatterns)
			{
				FRegexMatcher Matcher(Pattern, ClassName);
				if (Matcher.FindNext())
				{
					UKismetSystemLibrary::RaiseScriptError(FString::Printf(
						TEXT("SetObjectProperties on '%s' (%s): class '%s' is on the list "
							"UToolsetRegistrySettings::SetObjectPropertiesBlockedClasses and cannot be set"),
						*Object->GetPathName(), *ObjectClass->GetName(), *ClassName));
					return false;
				}
			}
		}
	}

	// Nested-key coverage carve-out: ImportPropertyWithNotify only surfaces unmatched JSON keys
	// when it recurses into a struct via ImportStructFieldsWithNotify. Paths that fall through
	// to EmitValueSet - FObjectProperty (instanced subobjects), FInstancedStruct,
	// IJsonObjectStructConverter implementors, and structs whose JSON object has no FProperty
	// matches - go straight through FJsonObjectConverter, which silently ignores unknown fields.
	// Misspelled fields under those paths are therefore NOT surfaced as InaccessibleProperties.
	const bool bObjectIsTemplate = PropertyAccessUtil::IsObjectTemplate(Object);
	TArray<UE::FSharedString> InaccessibleProperties;
	for (const TPair<UE::FSharedString, TSharedPtr<FJsonValue>>& Pair : PropertyData->Values)
	{
		const UE::FSharedString& Key = Pair.Key;
		FProperty* Property = PropertyAccessUtil::FindPropertyByName(FName(Key), ObjectClass);
		if (!Property)
		{
			InaccessibleProperties.Add(Key);
			continue;
		}
		const EPropertyAccessResultFlags AccessResult = PropertyAccessUtil::CanSetPropertyValue(
			Property, PropertyAccessUtil::EditorReadOnlyFlags, bObjectIsTemplate);
		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::PermissionDenied))
		{
			InaccessibleProperties.Add(Key);
			continue;
		}

		// Block list: test "ClassName.PropertyName" against the configured
		// property block patterns for every class in the inheritance chain.
		if (!PropertyBlockPatterns.IsEmpty())
		{
			bool bBlockedThisKey = false;
			for (const UClass* Class = ObjectClass; Class && !bBlockedThisKey; Class = Class->GetSuperClass())
			{
				const FString Composed = FString::Printf(TEXT("%s.%s"), *Class->GetName(), *Key);
				for (const FRegexPattern& Pattern : PropertyBlockPatterns)
				{
					FRegexMatcher Matcher(Pattern, Composed);
					if (Matcher.FindNext())
					{
						bBlockedThisKey = true;
						break;
					}
				}
			}
			if (bBlockedThisKey)
			{
				InaccessibleProperties.Add(Key);
				continue;
			}
		}

		void* SettingsPropPtr = Property->ContainerPtrToValuePtr<void>(Object);

		TArray<FString> NestedUnmatchedKeys;
		FPropertyImportContext Ctx;
		Ctx.Object = Object;
		Ctx.Chain.Add(Property);
		// PropertyPath must be kept parallel to Chain so the path builder includes the
		// top-level property name. Without this seed, nested unmatched-key paths would
		// start one level too shallow (e.g. "doesNotExist" instead of "StructProp.doesNotExist").
		Ctx.PropertyPath = Property->GetName();
		Ctx.bBypassContainerChecking = (BypassContainerCheck == EBypassContainerCheck::Yes);
		Ctx.OutUnmatchedKeys = &NestedUnmatchedKeys;

		const bool bWriteSucceeded =
			ImportPropertyWithNotify(PropertyData->TryGetField(Key), Property, SettingsPropPtr, Ctx);

		for (const FString& UnmatchedPath : NestedUnmatchedKeys)
		{
			InaccessibleProperties.Add(UE::FSharedString(UnmatchedPath));
		}

		if (!bWriteSucceeded)
		{
			// Only add the top-level key itself if nothing more specific has already been recorded.
			if (NestedUnmatchedKeys.IsEmpty())
			{
			InaccessibleProperties.Add(Key);
		}
			// Note: when bWriteSucceeded is false but NestedUnmatchedKeys is non-empty, sibling
			// fields inside the same struct may have been partially applied. We deliberately
			// DO NOT add such keys to OutSetPropertyNames - the function returns false from the
			// error path below, and callers should treat OutSetPropertyNames as authoritative
			// only on overall success. Under-report is the safer side of this contract.
		}
		else
		{
			OutSetPropertyNames.Add(FName(*Key));
		}
	}
	if (!InaccessibleProperties.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("SetObjectProperties on '%s' (%s): the following properties could not be set: %s"),
			*Object->GetPathName(), *Object->GetClass()->GetName(),
			*FString::Join(InaccessibleProperties, TEXT(", "))));
		return false;
	}
	return true;
}

TArray<FSoftClassPath> UToolsetLibrary::GetDerivedClasses(UClass* BaseClass)
{
	TArray<FSoftClassPath> DerivedClasses;
	if (!BaseClass)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Null base class provided"));
		return DerivedClasses;
	}

	FAssetRegistryModule& AssetRegistryModule =
		FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	TSet<FTopLevelAssetPath> DerivedNames;
	AssetRegistry.GetDerivedClassNames({ BaseClass->GetClassPathName() }, {}, DerivedNames);

	for (const FTopLevelAssetPath& DerivedName : DerivedNames)
	{
		// We ignore skeleton classes (contained in BPs) but consider all other subclasses.
		if (!DerivedName.GetAssetName().ToString().StartsWith(TEXT("SKEL_")))
		{
			FString PathName = DerivedName.ToString();
			DerivedClasses.Add(FSoftClassPath(PathName));
		}
	}
	return DerivedClasses;
}

TArray<UScriptStruct*> UToolsetLibrary::GetDerivedStructs(UScriptStruct* BaseStruct)
{
	TArray<UScriptStruct*> DerivedStructs;
	if (!BaseStruct)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Null base struct provided"));
		return DerivedStructs;
	}

	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		UScriptStruct* Struct = *It;
		if (Struct != BaseStruct && Struct->IsChildOf(BaseStruct))
		{
			DerivedStructs.Add(Struct);
		}
	}
	return DerivedStructs;
}

bool UToolsetLibrary::UndoTransaction(bool bCanRedo)
{
	if (!GEditor)
	{
		return false;
	}
	return GEditor->UndoTransaction(bCanRedo);
}

int32 UToolsetLibrary::GetActiveUndoCount()
{
	if (!GEditor || !GEditor->Trans)
	{
		return 0;
	}
	// Entries above the redo split: total queue length minus the portion
	// that has been undone (and is therefore sitting in the redo half).
	return GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount();
}
