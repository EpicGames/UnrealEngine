// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetLibraryImpl.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "JsonObjectStructInterface.h"
#include "Misc/AssertionMacros.h"
#include "UObject/PropertyAccessUtil.h"
#include "UObject/UnrealType.h"

#include "Kismet/KismetSystemLibrary.h"
#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/ToolsetJson.h"

namespace UE::ToolsetRegistry::Internal
{

// Forward declarations — ImportPropertyWithNotify and ImportStructFieldsWithNotify are mutually recursive.
bool ImportPropertyWithNotify(
	const TSharedPtr<FJsonValue>& NewJson,
	FProperty* Property,
	void* PropertyMemory,
	FPropertyImportContext Ctx);

bool ImportStructFieldsWithNotify(
	const TSharedPtr<FJsonObject>& NewJsonObj,
	const UStruct* Struct,
	void* StructMemory,
	FPropertyImportContext Ctx);

// ---------------------------------------------------------------------------
// Private helpers
// ---------------------------------------------------------------------------

TUniquePtr<FPropertyAccessChangeNotify> BuildNotify(
	const FPropertyImportContext& Ctx,
	EPropertyChangeType::Type ChangeType,
	const TMap<FString, int32>& ElementIndices)
{
	check(!Ctx.Chain.IsEmpty() && Ctx.Object);
#if WITH_EDITOR
	TUniquePtr<FPropertyAccessChangeNotify> Notify = MakeUnique<FPropertyAccessChangeNotify>();
	Notify->ChangeType = ChangeType;
	Notify->ChangedObject = Ctx.Object;
	Notify->NotifyMode = EPropertyAccessChangeNotifyMode::Always;
	Notify->ElementIndicesMap = ElementIndices;

	for (FProperty* P : Ctx.Chain)
	{
		Notify->ChangedPropertyChain.AddTail(P);
	}

	// Head = outermost UObject member; tail = innermost active property.
	Notify->ChangedPropertyChain.SetActiveMemberPropertyNode(Ctx.Chain[0]);
	Notify->ChangedPropertyChain.SetActivePropertyNode(Ctx.Chain.Last());
	return Notify;
#else
	return nullptr;
#endif
}

// Result of a structural diff between two container snapshots.
// Pure means the change can be applied as a clean sequence of adds (Delta > 0) or
// removes (Delta < 0) — no element mutations alongside the size change. Mixed
// means the change cannot be classified that way and must be rejected.
struct FContainerDiff
{
	enum class EKind { Pure, Mixed };
	EKind Kind = EKind::Mixed;
	// Populated for array diffs: positions in NewArray (when growing) or OldArray (when shrinking).
	TArray<int32> Indices;
	// Populated for map diffs: keys added (when growing) or removed (when shrinking).
	TArray<FString> Keys;
};

// Diffs OldArray vs NewArray. Sign of Delta selects direction:
//   Delta > 0: returns inserted positions in NewArray. Pure iff OldArray is a subsequence of NewArray.
//   Delta < 0: returns removed positions in OldArray. Pure iff NewArray is a subsequence of OldArray.
//   Delta == 0: returns Pure with empty Indices iff every element matches; Mixed otherwise.
FContainerDiff DiffArray(
	const TArray<TSharedPtr<FJsonValue>>& OldArray,
	const TArray<TSharedPtr<FJsonValue>>& NewArray)
{
	FContainerDiff Result;
	const int32 Delta = NewArray.Num() - OldArray.Num();

	if (Delta >= 0)
	{
		// Greedy subsequence match: walk both arrays, advancing OldIdx on equal pairs.
		TArray<int32> Inserted;
		int32 OldIdx = 0;
		for (int32 NewIdx = 0; NewIdx < NewArray.Num(); ++NewIdx)
		{
			if (OldIdx < OldArray.Num() &&
				NewArray[NewIdx].IsValid() && OldArray[OldIdx].IsValid() &&
				FJsonValue::CompareEqual(*NewArray[NewIdx], *OldArray[OldIdx]))
			{
				++OldIdx;
			}
			else
			{
				Inserted.Add(NewIdx);
			}
		}
		if (OldIdx == OldArray.Num() && Inserted.Num() == Delta)
		{
			Result.Kind = FContainerDiff::EKind::Pure;
			Result.Indices = MoveTemp(Inserted);
		}
		return Result;
	}

	// Delta < 0: pure-remove check (NewArray must be a subsequence of OldArray).
	TArray<int32> Removed;
	int32 NewIdx = 0;
	for (int32 OldIdx = 0; OldIdx < OldArray.Num(); ++OldIdx)
	{
		if (NewIdx < NewArray.Num() &&
			OldArray[OldIdx].IsValid() && NewArray[NewIdx].IsValid() &&
			FJsonValue::CompareEqual(*OldArray[OldIdx], *NewArray[NewIdx]))
		{
			++NewIdx;
		}
		else
		{
			Removed.Add(OldIdx);
		}
	}
	if (NewIdx == NewArray.Num() && Removed.Num() == -Delta)
	{
		Result.Kind = FContainerDiff::EKind::Pure;
		Result.Indices = MoveTemp(Removed);
	}
	return Result;
}

// Diffs OldObj vs NewObj. Sign of Delta selects direction:
//   Delta > 0: returns added keys. Pure iff no keys were removed.
//   Delta < 0: returns removed keys. Pure iff no keys were added.
//   Delta == 0: returns Pure with empty Keys iff the key sets match; Mixed otherwise.
FContainerDiff DiffMap(const FJsonObject& OldObj, const FJsonObject& NewObj)
{
	FContainerDiff Result;

	TArray<FString> Added;
	for (const TPair<UE::FSharedString, TSharedPtr<FJsonValue>>& Pair : NewObj.Values)
	{
		if (!OldObj.HasField(Pair.Key.ToView()))
		{
			Added.Add(FString(Pair.Key.ToView()));
		}
	}
	TArray<FString> Removed;
	for (const TPair<UE::FSharedString, TSharedPtr<FJsonValue>>& Pair : OldObj.Values)
	{
		if (!NewObj.HasField(Pair.Key.ToView()))
		{
			Removed.Add(FString(Pair.Key.ToView()));
		}
	}

	const int32 Delta = NewObj.Values.Num() - OldObj.Values.Num();
	if (Delta > 0 && Removed.IsEmpty())
	{
		Result.Kind = FContainerDiff::EKind::Pure;
		Result.Keys = MoveTemp(Added);
	}
	else if (Delta < 0 && Added.IsEmpty())
	{
		Result.Kind = FContainerDiff::EKind::Pure;
		Result.Keys = MoveTemp(Removed);
	}
	else if (Delta == 0 && Added.IsEmpty() && Removed.IsEmpty())
	{
		Result.Kind = FContainerDiff::EKind::Pure;
	}
	return Result;
}

// Returns the logical index of the map entry whose serialized key string matches KeyString,
// or INDEX_NONE if not found.
int32 FindMapLogicalIndexByKeyString(
	const FMapProperty* MapProp, void* MapMemory, const FString& KeyString)
{
	FScriptMapHelper Helper(MapProp, MapMemory);
	for (FScriptMapHelper::FIterator It(Helper); It; ++It)
	{
		FString ThisKey;
		MapProp->KeyProp->ExportTextItem_Direct(
			ThisKey, Helper.GetKeyPtr(It), nullptr, nullptr, PPF_None);
		if (ThisKey == KeyString)
		{
			return It.GetLogicalIndex();
		}
	}
	return INDEX_NONE;
}

// Returns false for struct types that use a custom JSON format not mapping field-name → FProperty.
bool ShouldRecurseIntoStruct(const FStructProperty* StructProp)
{
	const UScriptStruct* Struct = StructProp->Struct;
	// IJsonObjectStructConverter implementors manage their own JSON format.
	if (FJsonObjectStructInterfaceRegistry::HasStructConverterRegistered(Struct))
	{
		return false;
	}
	// FInstancedStruct and FInstancedPropertyBag use type-discriminator JSON formats.
	const FName StructName = Struct->GetFName();
	if (StructName == FName(TEXT("InstancedStruct")) || StructName == FName(TEXT("InstancedPropertyBag")))
	{
		return false;
	}
	// Also skip if ToolsetJson has a registered converter for this property —
	// it owns the import format and field-by-field recursion would bypass it.
	if (ToolsetJson::HasCustomImporter(StructProp))
	{
		return false;
	}
	return true;
}

// Returns true if any JSON key in JsonObj resolves to an FProperty on Struct.
// UUserDefinedStruct (Blueprint) member FNames carry auto-generated suffixes, so
// FindFProperty by the user-visible JSON key returns null and field-by-field
// recursion would silently fail. Callers gate ImportStructFieldsWithNotify on this.
bool JsonHasMatchingStructField(
	const TSharedPtr<FJsonObject>& JsonObj,
	const UScriptStruct* Struct)
{
	for (const TPair<FSharedString, TSharedPtr<FJsonValue>>& Pair : JsonObj->Values)
	{
		if (FindFProperty<FProperty>(Struct, FName(*Pair.Key)))
		{
			return true;
		}
	}
	return false;
}

// Returns a pointer to the LogicalIndex-th valid element in an Array or Set container.
void* GetContainerElementPtr(FProperty* ContainerProp, void* ContainerMemory, int32 LogicalIndex)
{
	if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(ContainerProp))
	{
		return FScriptArrayHelper(ArrayProp, ContainerMemory).GetRawPtr(LogicalIndex);
	}
	if (const FSetProperty* SetProp = CastField<FSetProperty>(ContainerProp))
	{
		return FScriptSetHelper(SetProp, ContainerMemory).FindNthElementPtr(LogicalIndex);
	}
	checkNoEntry(); // GetContainerElementPtr only supports Array and Set containers.
	return nullptr;
}

// Emits Pre, writes NewJson into the property via JsonDataToProperty, emits Post.
// Performs exactly one leaf write — by construction does NOT call
// ImportPropertyWithNotify or ImportStructFieldsWithNotify, so notifications
// cannot nest. Callers whose Pre and Post require different ElementIndices
// (the map-add path) must keep their explicit emission code.
bool EmitNotifyAndApplyJson(
	const FPropertyImportContext& Ctx,
	EPropertyChangeType::Type ChangeType,
	const TMap<FString, int32>& Indices,
	const TSharedPtr<FJsonValue>& NewJson,
	FProperty* Property,
	void* PropertyMemory)
{
	using namespace UE::ToolsetRegistry::Internal::ToolsetJson;
	TUniquePtr<FPropertyAccessChangeNotify> Notify = BuildNotify(Ctx, ChangeType, Indices);
	PropertyAccessUtil::EmitPreChangeNotify(Notify.Get(), false);
	const bool bOk = JsonDataToProperty(NewJson, Property, PropertyMemory, Ctx.Object);
	PropertyAccessUtil::EmitPostChangeNotify(Notify.Get(), false);
	return bOk;
}

// Emits ValueSet Pre/Post around a JsonDataToProperty call. Used as the default fallback path.
bool EmitValueSet(
	const TSharedPtr<FJsonValue>& NewJson,
	FProperty* Property,
	void* PropertyMemory,
	const FPropertyImportContext& Ctx)
{
	return EmitNotifyAndApplyJson(
		Ctx, EPropertyChangeType::ValueSet, Ctx.ElementIndices,
		NewJson, Property, PropertyMemory);
}

// Emits the appropriate per-element notification for a single container element at
// LogicalIndex. Skips when Old == New. Recurses into struct fields when the element is a
// struct whose JSON is an object; otherwise emits ValueSet for the element in place.
//
// MapKeyString is the JSON key when ContainerProp is a map (empty otherwise). Used to
// stamp "[KeyString]" into the path for error reporting; ElementIndices still receives the
// logical iteration index so the change-notification framework gets the form it expects.
bool HandleContainerElement(
	const TSharedPtr<FJsonValue>& NewElem,
	const TSharedPtr<FJsonValue>& OldElem,
	FProperty* ContainerProp,
	FProperty* ElemProp,
	void* ElemMem,
	int32 LogicalIndex,
	const FString& MapKeyString,
	const FPropertyImportContext& Ctx)
{
	if (OldElem.IsValid() && NewElem.IsValid() && FJsonValue::CompareEqual(*OldElem, *NewElem))
	{
		return true;
	}

	if (const FStructProperty* ElemStructProp = CastField<FStructProperty>(ElemProp))
	{
		const TSharedPtr<FJsonObject>* ElemJsonObj = nullptr;
		if (NewElem.IsValid() && NewElem->TryGetObject(ElemJsonObj) && ShouldRecurseIntoStruct(ElemStructProp)
			&& JsonHasMatchingStructField(*ElemJsonObj, ElemStructProp->Struct))
		{
			FPropertyImportContext ElemCtx = Ctx;
			ElemCtx.ElementIndices.Add(ContainerProp->GetName(), LogicalIndex);
			ElemCtx.PropertyPath += MapKeyString.IsEmpty()
				? FString::Printf(TEXT("[%d]"), LogicalIndex)
				: FString::Printf(TEXT("[%s]"), *MapKeyString);
			return ImportStructFieldsWithNotify(*ElemJsonObj, ElemStructProp->Struct, ElemMem, ElemCtx);
		}
		// Struct element with custom importer, mangled-name UDS, or non-object JSON:
		// fall through to ValueSet so FJsonObjectConverter handles it.
	}

	TMap<FString, int32> Indices = Ctx.ElementIndices;
	Indices.Add(ContainerProp->GetName(), LogicalIndex);
	return EmitNotifyAndApplyJson(
		Ctx, EPropertyChangeType::ValueSet, Indices,
		NewElem, ElemProp, ElemMem);
}

// Handles the delta==0 array/set case: recurses into changed elements to find nested
// container size changes, emitting element-level notifications.
bool HandleArrayElementRecursion(
	const TArray<TSharedPtr<FJsonValue>>& NewArray,
	const TArray<TSharedPtr<FJsonValue>>& OldArray,
	FProperty* ContainerProp,
	void* ContainerMemory,
	const FPropertyImportContext& Ctx)
{
	FProperty* ElemProp = ContainerProp->IsA<FArrayProperty>()
		? CastField<FArrayProperty>(ContainerProp)->Inner
		: CastField<FSetProperty>(ContainerProp)->ElementProp;

	bool bAllOk = true;
	for (int32 i = 0; i < OldArray.Num(); ++i)
	{
		void* ElemMem = GetContainerElementPtr(ContainerProp, ContainerMemory, i);
		if (!HandleContainerElement(NewArray[i], OldArray[i], ContainerProp, ElemProp, ElemMem,
				i, FString(), Ctx))
		{
			bAllOk = false;
		}
	}
	return bAllOk;
}

// Emits an ArrayClear notification and writes the (empty) new state into the property.
// Used for both empty-array and empty-map clears.
bool HandleContainerClear(
	const TSharedPtr<FJsonValue>& NewJson,
	FProperty* Property,
	void* PropertyMemory,
	const FPropertyImportContext& Ctx)
{
	return EmitNotifyAndApplyJson(
		Ctx, EPropertyChangeType::ArrayClear, Ctx.ElementIndices,
		NewJson, Property, PropertyMemory);
}

// Applies pure-add insertions incrementally, emitting one ArrayAdd per element.
// Working stays in sync with the live array so that instanced sub-object outers
// are correct at each step.
bool HandleArrayAdds(
	const TArray<TSharedPtr<FJsonValue>>& NewArray,
	const TArray<TSharedPtr<FJsonValue>>& OldArray,
	const TArray<int32>& InsertedIndices,
	FProperty* Property,
	void* PropertyMemory,
	const FPropertyImportContext& Ctx)
{
	TArray<TSharedPtr<FJsonValue>> Working = OldArray;
	for (const int32 Idx : InsertedIndices)
	{
		Working.Insert(NewArray[Idx], Idx);
		TMap<FString, int32> Indices = Ctx.ElementIndices;
		Indices.Add(Property->GetName(), Idx);
		if (!EmitNotifyAndApplyJson(
				Ctx, EPropertyChangeType::ArrayAdd, Indices,
				MakeShared<FJsonValueArray>(Working), Property, PropertyMemory))
		{
			return false;
		}
	}
	return true;
}

// Applies pure-remove deletions incrementally, emitting one ArrayRemove per element.
// Processes highest index first so lower indices are never shifted by earlier removals.
bool HandleArrayRemoves(
	const TArray<TSharedPtr<FJsonValue>>& OldArray,
	const TArray<int32>& RemovedIndices,
	FProperty* Property,
	void* PropertyMemory,
	const FPropertyImportContext& Ctx)
{
	TArray<TSharedPtr<FJsonValue>> Working = OldArray;
	for (int32 k = RemovedIndices.Num() - 1; k >= 0; --k)
	{
		const int32 Idx = RemovedIndices[k];
		Working.RemoveAt(Idx);
		TMap<FString, int32> Indices = Ctx.ElementIndices;
		Indices.Add(Property->GetName(), Idx);
		if (!EmitNotifyAndApplyJson(
				Ctx, EPropertyChangeType::ArrayRemove, Indices,
				MakeShared<FJsonValueArray>(Working), Property, PropertyMemory))
		{
			return false;
		}
	}
	return true;
}

// Dispatches array/set size changes to clear / adds / removes after a pure-vs-mixed diff.
//
// FSet handling caveat: TSet iteration order is unspecified, so the pure-add / pure-remove
// detection assumes the incoming JSON array is ordered to match the live set's iteration
// order. When the orders disagree, DiffArray returns Mixed and the operation is rejected.
// Callers that need order-independent set updates should pass EBypassContainerCheck::Yes,
// which skips the diff entirely and emits a single ValueSet for the property.
bool HandleArraySizeChange(
	const TSharedPtr<FJsonValue>& NewJson,
	const TArray<TSharedPtr<FJsonValue>>& NewArray,
	const TArray<TSharedPtr<FJsonValue>>& OldArray,
	FProperty* Property,
	void* PropertyMemory,
	const FPropertyImportContext& Ctx)
{
	if (NewArray.Num() == 0)
	{
		return HandleContainerClear(NewJson, Property, PropertyMemory, Ctx);
	}

	const FContainerDiff Diff = DiffArray(OldArray, NewArray);
	const bool bGrowing = NewArray.Num() > OldArray.Num();
	if (Diff.Kind != FContainerDiff::EKind::Pure)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("SetObjectProperties: property '%s' — Array%s: elements changed alongside "
				 "the size change; %s are ambiguous."),
			*Property->GetName(),
			bGrowing ? TEXT("Add") : TEXT("Remove"),
			bGrowing ? TEXT("insertion points") : TEXT("removed elements")));
		return false;
	}

	return bGrowing
		? HandleArrayAdds(NewArray, OldArray, Diff.Indices, Property, PropertyMemory, Ctx)
		: HandleArrayRemoves(OldArray, Diff.Indices, Property, PropertyMemory, Ctx);
}

// Recurses into kept map entries (keys present in both Old and New). For each kept key
// whose value changed, emits a per-value notification via HandleContainerElement (struct
// recursion or ValueSet on the value in place). Mirrors HandleArrayElementRecursion.
bool HandleMapElementRecursion(
	const FJsonObject& NewJsonObj,
	const FJsonObject& OldJsonObj,
	FMapProperty* MapProp,
	void* PropertyMemory,
	const FPropertyImportContext& Ctx)
{
	FProperty* ValueProp = MapProp->ValueProp;
	FScriptMapHelper Helper(MapProp, PropertyMemory);

	bool bAllOk = true;
	for (FScriptMapHelper::FIterator It(Helper); It; ++It)
	{
		FString ThisKey;
		MapProp->KeyProp->ExportTextItem_Direct(
			ThisKey, Helper.GetKeyPtr(It), nullptr, nullptr, PPF_None);

		const TSharedPtr<FJsonValue> NewValue = NewJsonObj.TryGetField(ThisKey);
		const TSharedPtr<FJsonValue> OldValue = OldJsonObj.TryGetField(ThisKey);
		if (!NewValue || !OldValue)
		{
			// Key is being added or removed — handled by the structural-change path.
			continue;
		}

		if (!HandleContainerElement(NewValue, OldValue, MapProp, ValueProp,
			Helper.GetValuePtr(It), It.GetLogicalIndex(), ThisKey, Ctx))
		{
			bAllOk = false;
		}
	}
	return bAllOk;
}

// Applies pure-add map insertions: updates kept-key values first (so the live map matches
// NewJsonObj for all kept keys), then applies each new entry one at a time. The per-iteration
// ArrayAdd is the one notification site that cannot use EmitNotifyAndApplyJson because the
// new entry's logical index — and therefore the Post-emission's ElementIndices — is only
// knowable after the import.
bool HandleMapAdds(
	const FJsonObject& NewJsonObj,
	const FJsonObject& OldJsonObj,
	const TArray<FString>& AddedKeys,
	FMapProperty* MapProp,
	void* PropertyMemory,
	const FPropertyImportContext& Ctx)
{
	using namespace UE::ToolsetRegistry::Internal::ToolsetJson;

	bool bAllOk = HandleMapElementRecursion(NewJsonObj, OldJsonObj, MapProp, PropertyMemory, Ctx);

	// Build Working with the (now-updated) kept entries so subsequent JsonDataToProperty
	// calls preserve their new values.
	TSharedPtr<FJsonObject> Working = MakeShared<FJsonObject>();
	for (const TPair<UE::FSharedString, TSharedPtr<FJsonValue>>& Pair : NewJsonObj.Values)
	{
		if (OldJsonObj.HasField(Pair.Key.ToView()))
		{
			Working->SetField(Pair.Key, Pair.Value);
		}
	}

	for (const FString& AddedKey : AddedKeys)
	{
		Working->SetField(AddedKey, NewJsonObj.TryGetField(AddedKey));
		// Pre: entry doesn't exist yet so no index is known; emit without element index.
		TUniquePtr<FPropertyAccessChangeNotify> PreNotify =
			BuildNotify(Ctx, EPropertyChangeType::ArrayAdd, Ctx.ElementIndices);
		PropertyAccessUtil::EmitPreChangeNotify(PreNotify.Get(), false);
		const bool bOk = JsonDataToProperty(
			MakeShared<FJsonValueObject>(Working), MapProp, PropertyMemory, Ctx.Object);
		// Post: find logical index of the new entry in the post-import map.
		const int32 LogicalIdx =
			FindMapLogicalIndexByKeyString(MapProp, PropertyMemory, AddedKey);
		ensureMsgf(LogicalIdx != INDEX_NONE,
			TEXT("SetObjectProperties: added key '%s' not found in map '%s' after import."),
			*AddedKey, *MapProp->GetName());
		TMap<FString, int32> PostIndices = Ctx.ElementIndices;
		if (LogicalIdx != INDEX_NONE)
		{
			PostIndices.Add(MapProp->GetName(), LogicalIdx);
		}
		TUniquePtr<FPropertyAccessChangeNotify> PostNotify =
			BuildNotify(Ctx, EPropertyChangeType::ArrayAdd, PostIndices);
		PropertyAccessUtil::EmitPostChangeNotify(PostNotify.Get(), false);
		if (!bOk)
		{
			return false;
		}
	}
	return bAllOk;
}

// Applies pure-remove map deletions one at a time, then updates remaining-key values
// via per-element recursion.
bool HandleMapRemoves(
	const FJsonObject& NewJsonObj,
	const FJsonObject& OldJsonObj,
	const TArray<FString>& RemovedKeys,
	FMapProperty* MapProp,
	void* PropertyMemory,
	const FPropertyImportContext& Ctx)
{
	TSharedPtr<FJsonObject> Working = MakeShared<FJsonObject>();
	Working->Values = OldJsonObj.Values;
	for (const FString& RemovedKey : RemovedKeys)
	{
		// Find logical index BEFORE removal (entry still exists at this point).
		const int32 LogicalIdx =
			FindMapLogicalIndexByKeyString(MapProp, PropertyMemory, RemovedKey);
		TMap<FString, int32> Indices = Ctx.ElementIndices;
		if (LogicalIdx != INDEX_NONE)
		{
			Indices.Add(MapProp->GetName(), LogicalIdx);
		}
		Working->RemoveField(RemovedKey);
		if (!EmitNotifyAndApplyJson(
				Ctx, EPropertyChangeType::ArrayRemove, Indices,
				MakeShared<FJsonValueObject>(Working), MapProp, PropertyMemory))
		{
			return false;
		}
	}

	// After removes, update remaining-entry values via per-element recursion.
	return HandleMapElementRecursion(NewJsonObj, OldJsonObj, MapProp, PropertyMemory, Ctx);
}

// Dispatches map changes to recursion / clear / adds / removes after a pure-vs-mixed diff.
// Kept-key value changes are emitted before adds and after removes; structural changes
// are applied incrementally.
bool HandleMapChange(
	const TSharedPtr<FJsonValue>& NewJson,
	const FJsonObject& NewJsonObj,
	const FJsonObject& OldJsonObj,
	FMapProperty* MapProp,
	void* PropertyMemory,
	const FPropertyImportContext& Ctx)
{
	const int32 Delta = NewJsonObj.Values.Num() - OldJsonObj.Values.Num();
	const FContainerDiff Diff = DiffMap(OldJsonObj, NewJsonObj);

	if (Delta == 0)
	{
		if (Diff.Kind != FContainerDiff::EKind::Pure)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("SetObjectProperties: property '%s' — keys swapped without size change; "
					 "changes are ambiguous."),
				*MapProp->GetName()));
			return false;
		}
		return HandleMapElementRecursion(NewJsonObj, OldJsonObj, MapProp, PropertyMemory, Ctx);
	}

	if (Delta > 0)
	{
		if (Diff.Kind != FContainerDiff::EKind::Pure)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("SetObjectProperties: property '%s' — MapAdd: keys removed alongside "
					 "the size increase; changes are ambiguous."),
				*MapProp->GetName()));
			return false;
		}
		return HandleMapAdds(NewJsonObj, OldJsonObj, Diff.Keys, MapProp, PropertyMemory, Ctx);
	}

	// Delta < 0.
	if (NewJsonObj.Values.Num() == 0)
	{
		return HandleContainerClear(NewJson, MapProp, PropertyMemory, Ctx);
	}
	if (Diff.Kind != FContainerDiff::EKind::Pure)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("SetObjectProperties: property '%s' — MapRemove: keys added alongside "
				 "the size decrease; changes are ambiguous."),
			*MapProp->GetName()));
		return false;
	}
	return HandleMapRemoves(NewJsonObj, OldJsonObj, Diff.Keys, MapProp, PropertyMemory, Ctx);
}

// Builds the dotted path for an unmatched JSON key encountered inside a nested struct.
// Uses Ctx.PropertyPath which has been grown at each Chain.Add() and container descent -
// so the returned path correctly distinguishes map keys from indices and is immune to
// property-name collisions across chain depths (the same field name appearing at
// multiple struct levels is no longer ambiguous).
FString BuildUnmatchedKeyPath(const FPropertyImportContext& Ctx, FStringView LeafKey)
{
	if (Ctx.PropertyPath.IsEmpty())
	{
		return FString(LeafKey);
	}
	return Ctx.PropertyPath + TEXT(".") + FString(LeafKey);
}

// ---------------------------------------------------------------------------
// Public API
// ---------------------------------------------------------------------------

// Dispatches each field of a JSON object into the struct, adding the field to the chain
// and recursing through ImportPropertyWithNotify for correct container notifications.
bool ImportStructFieldsWithNotify(
	const TSharedPtr<FJsonObject>& NewJsonObj,
	const UStruct* Struct,
	void* StructMemory,
	FPropertyImportContext Ctx) // by value — each frame owns its own copy
{
	bool bAllOk = true;
	for (const TPair<UE::FSharedString, TSharedPtr<FJsonValue>>& Pair : NewJsonObj->Values)
	{
		FProperty* FieldProp = FindFProperty<FProperty>(Struct, FName(*Pair.Key));
		if (!FieldProp)
		{
			if (Ctx.OutUnmatchedKeys)
			{
				Ctx.OutUnmatchedKeys->Add(BuildUnmatchedKeyPath(Ctx, Pair.Key.ToView()));
			}
			bAllOk = false;
			continue;
		}

		void* FieldMem = FieldProp->ContainerPtrToValuePtr<void>(StructMemory);
		FPropertyImportContext FieldCtx = Ctx;
		FieldCtx.Chain.Add(FieldProp);
		// Grow the dotted path as we descend a level. Container-element descents
		// in HandleContainerElement extend this further with "[Index]" / "[KeyString]".
		if (!FieldCtx.PropertyPath.IsEmpty())
		{
			FieldCtx.PropertyPath += TEXT(".");
		}
		FieldCtx.PropertyPath += FieldProp->GetName();

		if (!ImportPropertyWithNotify(Pair.Value, FieldProp, FieldMem, FieldCtx))
		{
			bAllOk = false;
		}
	}
	return bAllOk;
}

// Main dispatch: routes each property through its type-appropriate notification path.
// Struct, Array/Set, and Map containers are handled recursively; everything else emits ValueSet.
// FObjectProperty and special struct types (FInstancedStruct, IJsonObjectStructConverter
// implementors) fall through to ValueSet.
bool ImportPropertyWithNotify(
	const TSharedPtr<FJsonValue>& NewJson,
	FProperty* Property,
	void* PropertyMemory,
	FPropertyImportContext Ctx) // by value — each frame owns its own copy
{
	using namespace UE::ToolsetRegistry::Internal::ToolsetJson;

	if (!NewJson.IsValid())
	{
		return false;
	}

	if (Ctx.bBypassContainerChecking)
	{
		return EmitValueSet(NewJson, Property, PropertyMemory, Ctx);
	}

	if (const FStructProperty* StructProp = CastField<FStructProperty>(Property))
	{
		const TSharedPtr<FJsonObject>* JsonObj = nullptr;
		if (NewJson->TryGetObject(JsonObj) && ShouldRecurseIntoStruct(StructProp)
			&& JsonHasMatchingStructField(*JsonObj, StructProp->Struct))
		{
			return ImportStructFieldsWithNotify(*JsonObj, StructProp->Struct, PropertyMemory, Ctx);
		}
		// Struct with custom importer or mangled-name UDS: falls through to ValueSet below.
	}
	else if (Property->IsA<FArrayProperty>() || Property->IsA<FSetProperty>())
	{
		const TArray<TSharedPtr<FJsonValue>>* NewArray = nullptr;
		if (NewJson->TryGetArray(NewArray))
		{
			TSharedPtr<FJsonValue> OldJsonValue = PropertyToJsonData(Property, PropertyMemory);
			const TArray<TSharedPtr<FJsonValue>>* OldArray = nullptr;
			if (OldJsonValue.IsValid() && OldJsonValue->TryGetArray(OldArray))
			{
				const int32 Delta = NewArray->Num() - OldArray->Num();
				if (Delta == 0)
				{
					return HandleArrayElementRecursion(*NewArray, *OldArray, Property, PropertyMemory, Ctx);
				}
				return HandleArraySizeChange(NewJson, *NewArray, *OldArray, Property, PropertyMemory, Ctx);
			}
			// Old array failed to serialize — fall through to ValueSet.
		}
		// JSON is not an array — fall through to ValueSet.
	}
	else if (FMapProperty* MapProp = CastField<FMapProperty>(Property))
	{
		const TSharedPtr<FJsonObject>* NewJsonObj = nullptr;
		if (NewJson->TryGetObject(NewJsonObj))
		{
			TSharedPtr<FJsonValue> OldJsonValue = PropertyToJsonData(Property, PropertyMemory);
			const TSharedPtr<FJsonObject>* OldJsonObj = nullptr;
			if (OldJsonValue.IsValid() && OldJsonValue->TryGetObject(OldJsonObj))
			{
				return HandleMapChange(NewJson, **NewJsonObj, **OldJsonObj, MapProp, PropertyMemory, Ctx);
			}
			// Old map failed to serialize — fall through to ValueSet.
		}
		// JSON is not an object — fall through to ValueSet.
	}

	return EmitValueSet(NewJson, Property, PropertyMemory, Ctx);
}

}  // namespace UE::ToolsetRegistry::Internal
