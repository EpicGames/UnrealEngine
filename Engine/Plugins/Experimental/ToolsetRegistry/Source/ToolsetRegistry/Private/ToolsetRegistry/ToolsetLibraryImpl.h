// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

class FJsonValue;
class FProperty;

namespace UE::ToolsetRegistry::Internal
{

/**
 * Accumulated context passed down through recursive property import calls.
 * Each recursive frame receives its own copy so inner frames can extend the chain
 * and element-index map without affecting sibling calls.
 */
struct FPropertyImportContext
{
	/** Property chain, outermost-to-innermost. Chain[0] is always a direct UObject member. */
	TArray<FProperty*> Chain;
	/**
	 * Element indices accumulated as we descend into containers. Keyed by container property
	 * name; used to populate FPropertyAccessChangeNotify::ElementIndicesMap for the change
	 * notification framework. NOTE: this is a name-keyed flat map, so if the same property name
	 * appears at two different chain depths the inner overwrites the outer. That is acceptable
	 * for notifications (consumers only care about the leaf element). Path construction for
	 * error reporting uses PropertyPath instead, which is collision-free.
	 */
	TMap<FString, int32> ElementIndices;
	/**
	 * Pre-built dotted path of every Chain.Add() and container descent taken so far, e.g.
	 * "StructProp.Items[3].Tag" or "Mappings[KeyName]". Each push into Chain appends ".Name";
	 * each descent into a container element appends "[Index]" (arrays/sets) or "[KeyString]"
	 * (maps). Read by the unmatched-key emission to produce paths that survive name collisions
	 * across chain depths and that distinguish map keys from iteration indices.
	 */
	FString PropertyPath;
	UObject* Object = nullptr;
	/** When true, ImportPropertyWithNotify skips container-diff logic and always emits ValueSet. */
	bool bBypassContainerChecking = false;
	/**
	 * If non-null, ImportStructFieldsWithNotify appends dotted property paths here for any JSON
	 * key that didn't resolve to an FProperty on the nested struct. The top-level caller uses
	 * this to surface nested misspellings (e.g. "ShakeData.shake_class" instead of "ShakeClass")
	 * in the same error path as top-level unknown keys, rather than silently dropping them.
	 */
	TArray<FString>* OutUnmatchedKeys = nullptr;
};

/**
 * Imports NewJson into the property at PropertyMemory, emitting the correct Pre/Post change
 * notifications (ArrayAdd/ArrayRemove/ArrayClear/ValueSet) based on the detected change type.
 * Recurses into structs and same-size containers to find nested container changes.
 * FObjectProperty and special struct types (FInstancedStruct, IJsonObjectStructConverter
 * implementors) fall through to ValueSet.
 * TODO: Update JsonObjectConverter to track property and subobject chains and pass
 * this information to its ImportCb. We can then move this logic to the JsonObjectConverter
 * and our ImportCb overrides, making it more tractable to handle notifies within FObjectProperty, 
 * for example (UE-379502)
 */
bool ImportPropertyWithNotify(
	const TSharedPtr<FJsonValue>& NewJson,
	FProperty* Property,
	void* PropertyMemory,
	FPropertyImportContext Ctx);

}  // namespace UE::ToolsetRegistry::Internal
