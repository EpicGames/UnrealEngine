// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolsetRegistry.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Editor.h"
#include "Logging/LogVerbosity.h"
#include "Misc/Optional.h"
#include "Misc/Variant.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

#include "ToolsetRegistry/ColorConverter.h"
#include "ToolsetRegistry/GuidConverter.h"
#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/ReferenceConverter.h"
#include "ToolsetRegistry/TransformConverter.h"
#include "ToolsetRegistry/Toolset.h"

namespace UE::ToolsetRegistry
{
	FToolsetRegistry::FToolsetRegistry(
		const TArray<FString>& InBlockedNames,
		const TArray<FString>& InAllowedNames)
		: BlockedNames(InBlockedNames),
		  AllowedNames(InAllowedNames)
	{
		TSharedPtr<FToolsetColorConverter> ColorConverter = MakeShared<FToolsetColorConverter>();
		RegisterConverter(ColorConverter);

		TSharedPtr<FToolsetReferenceConverter> ReferenceConverter = MakeShared<FToolsetReferenceConverter>();
		RegisterConverter(ReferenceConverter);

		TSharedPtr<FToolsetTransformConverter> TransformConverter = MakeShared<FToolsetTransformConverter>();
		RegisterConverter(TransformConverter);

		TSharedPtr<FToolsetGuidConverter> GuidConverter = MakeShared<FToolsetGuidConverter>();
		RegisterConverter(GuidConverter);
	}

	FToolsetRegistry::~FToolsetRegistry() = default;

	TValueOrError<FToolDescriptor, FString> FToolDescriptor::FromString(
		const FString& ToolsetAndToolName, bool LogErrors)
	{
		FToolDescriptor Descriptor;
		if (!ToolsetAndToolName.Split(
			TEXT("."), &Descriptor.ToolsetName, &Descriptor.ToolName,
			ESearchCase::IgnoreCase, ESearchDir::FromEnd) ||
			Descriptor.ToolsetName.IsEmpty() || Descriptor.ToolName.IsEmpty())
		{
			FString Message = FString::Printf(
				TEXT("Invalid tool name format. Expected format: ToolsetName.ToolName, got %s"),
				*ToolsetAndToolName);
			if (LogErrors)
			{
				UE_LOGF(LogToolsetRegistry, Error, "%ls", *Message);
			}
			return MakeError(Message);
		}
		return MakeValue(Descriptor);
	}

	void FToolsetRegistry::ApplyFiltersToAllToolsets()
	{
		for (const auto& [ToolsetName, Toolset] : ToolsetHandlers)
		{
			Toolset->SetNameFilters(BlockedNames, AllowedNames);
		}
		OnToolsetRegistryChanged.Broadcast();
	}

	void FToolsetRegistry::AddBlockedName(const FString& Name)
	{
		if (Name.IsEmpty() || BlockedNames.Contains(Name))
		{
			return;
		}
		BlockedNames.Add(Name);
		ApplyFiltersToAllToolsets();
	}

	void FToolsetRegistry::RemoveBlockedName(const FString& Name)
	{
		if (BlockedNames.Remove(Name) == 0)
		{
			return;
		}
		ApplyFiltersToAllToolsets();
	}

	const TArray<FString>& FToolsetRegistry::GetBlockedNames() const
	{
		return BlockedNames;
	}

	void FToolsetRegistry::AddAllowedName(const FString& Name)
	{
		if (Name.IsEmpty() || AllowedNames.Contains(Name))
		{
			return;
		}
		AllowedNames.Add(Name);
		ApplyFiltersToAllToolsets();
	}

	void FToolsetRegistry::RemoveAllowedName(const FString& Name)
	{
		if (AllowedNames.Remove(Name) == 0)
		{
			return;
		}
		ApplyFiltersToAllToolsets();
	}

	const TArray<FString>& FToolsetRegistry::GetAllowedNames() const
	{
		return AllowedNames;
	}

	bool FToolsetRegistry::RegisterToolset(TSharedPtr<FToolset> ToolsetHandler)
	{
		if (!ToolsetHandler.IsValid())
		{
			UE_LOGF(LogToolsetRegistry, Warning, "Toolset must not be null");
			return false;
		}
		if (ToolsetHandlers.Contains(ToolsetHandler->GetToolsetName()))
		{
			UE_LOGF(
				LogToolsetRegistry, Warning, "Toolset '%ls' already registered.",
				*ToolsetHandler->GetToolsetName());
			return false;
		}
		ToolsetHandlers.Add(ToolsetHandler->GetToolsetName(), ToolsetHandler);

		ToolsetHandler->SetNameFilters(BlockedNames, AllowedNames);

		OnToolsetRegistryChanged.Broadcast();
		return true;
	}

	bool FToolsetRegistry::UnregisterToolset(TSharedPtr<FToolset> ToolsetHandler)
	{
		if (!ToolsetHandler.IsValid())
		{
			UE_LOGF(LogToolsetRegistry, Warning, "Toolset must not be null");
			return false;
		}
		if (ToolsetHandlers.Remove(ToolsetHandler->GetToolsetName()) == 0)
		{
			UE_LOGF(
				LogToolsetRegistry, Warning, "Toolset '%ls' not registered",
				*ToolsetHandler->GetToolsetName());
			return false;
		}
		OnToolsetRegistryChanged.Broadcast();
		return true;
	}

	TSharedPtr<FToolset> FToolsetRegistry::Find(
		const FString& ToolsetName, bool LogErrors, FString* ErrorMessage)
	{
		TSharedPtr<FToolset>* FoundToolsetHandler = ToolsetHandlers.Find(ToolsetName);
		if (FoundToolsetHandler != nullptr && (*FoundToolsetHandler)->IsEnabled())
		{
			return *FoundToolsetHandler;
		}

		if (LogErrors || ErrorMessage)
		{
			FString Message = FString::Printf(TEXT("Toolset '%s' not found"), *ToolsetName);
			if (LogErrors)
			{
				UE_LOGF(LogToolsetRegistry, Error, "%ls", *Message);
			}
			if (ErrorMessage != nullptr)
			{
				*ErrorMessage = Message;
			}
		}
		return nullptr;
	}

	TFuture<TValueOrError<FString, FString>> FToolsetRegistry::ExecuteTool(
		const FString& ToolsetAndToolName, const FString& JsonInput)
	{
		// Split the tool name to get the toolset name and the actual tool name.
		TValueOrError<FToolDescriptor, FString> ToolDescriptor =
			FToolDescriptor::FromString(ToolsetAndToolName);
		if (ToolDescriptor.HasError())
		{
			return MakeFulfilledPromise<TValueOrError<FString, FString>>(
				MakeError(ToolDescriptor.StealError())).GetFuture();
		}

		// Execute the tool.
		return ExecuteTool(ToolDescriptor.StealValue(), JsonInput);
	}

	TFuture<TValueOrError<FString, FString>> FToolsetRegistry::ExecuteTool(
		const FToolDescriptor& ToolDescriptor, const FString& JsonInput)
	{
		TSharedPtr<FToolset>* FoundToolsetHandler = ToolsetHandlers.Find(ToolDescriptor.ToolsetName);
		if (!FoundToolsetHandler)
		{
			FString Message = FString::Printf(
				TEXT("Toolset '%s' not found"), *ToolDescriptor.ToolsetName);
			UE_LOGF(LogToolsetRegistry, Error, "%ls", *Message);
			return MakeFulfilledPromise<TValueOrError<FString, FString>>(
				MakeError(Message)).GetFuture();
		}

		// Execute the tool.
		return (*FoundToolsetHandler)->ExecuteTool(ToolDescriptor.ToolName, JsonInput);
	}

	void FToolsetRegistry::ForEachToolset(
		TFunctionRef<void(const FString& Name, const FToolset& Toolset)> Callback) const
	{
		for (const TPair<FString, TSharedPtr<FToolset>>& Pair : ToolsetHandlers)
		{
			if (Pair.Value.IsValid() && Pair.Value->IsEnabled())
			{
				Callback(Pair.Key, *Pair.Value);
			}
		}
	}

	FString FToolsetRegistry::GetToolsetJsonSchemas() const
	{
		FString Result = TEXT("[");
		bool bFirst = true;
		for (const auto& [Name, Toolset]: ToolsetHandlers)
		{
			FString Schema = Toolset->GetJsonSchema();
			if (Schema.IsEmpty()) continue;

			if (!bFirst) Result += TEXT(",");
			bFirst = false;
			Result += Schema;
		}
		Result += TEXT("]");
		return Result;
	}

	bool FToolsetRegistry::RegisterConverter(TSharedPtr<FToolsetJsonConverter> JsonConverter)
	{
		verify(JsonConverter.IsValid());
		verify(!JsonConverters.Contains(JsonConverter->GetName()));
		JsonConverters.Add(JsonConverter->GetName(), JsonConverter);
		return true;
	}

	bool FToolsetRegistry::UnregisterConverter(TSharedPtr<FToolsetJsonConverter> JsonConverter)
	{
		verify(JsonConverter.IsValid());
		verify(JsonConverters.Remove(JsonConverter->GetName()));
		return true;
	}

	TSharedPtr<FToolsetJsonConverter> FToolsetRegistry::GetConverterForProperty(
		TNotNull<const FProperty*> Property)
	{
		for (const auto& Converter : JsonConverters)
		{
			if (Converter.Value->CanConvertProperty(Property))
			{
				return Converter.Value;
			}
		}
		return nullptr;
	}
}
