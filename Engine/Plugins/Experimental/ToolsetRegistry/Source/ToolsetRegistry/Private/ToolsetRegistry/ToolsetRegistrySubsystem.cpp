// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"

#include "Editor.h"
#include "Subsystems/SubsystemCollection.h"
#include "UObject/UObjectGlobals.h"

#include "ToolsetRegistry/AgentSkill.h"
#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

TValueOrError<TObjectPtr<UToolsetRegistrySubsystem>, FString> UToolsetRegistrySubsystem::Get(
	const FString& WarningMessage)
{
	// Local function to construct error to return and optionally log a warning.
	auto MaybeWarnOnError = [&WarningMessage](const FString& ErrorMessage)
		{
			if (!WarningMessage.IsEmpty())
			{
				UE_LOGF(LogToolsetRegistry, Warning, "%ls: %ls", *WarningMessage, *ErrorMessage);
			}
			return MakeError(ErrorMessage);
		};

	if (!GEditor)
	{
		return MaybeWarnOnError(TEXT("Editor is not available"));
	}

	TObjectPtr<UToolsetRegistrySubsystem> ToolsetRegistrySubsystem(
		GEditor->GetEditorSubsystem<UToolsetRegistrySubsystem>());

	if (!ToolsetRegistrySubsystem)
	{
		return MaybeWarnOnError(TEXT("ToolsetRegistrySubsystem subsystem is not available"));
	}

	return MakeValue(ToolsetRegistrySubsystem);
}

void UToolsetRegistrySubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	const UToolsetRegistrySettings* Settings = GetDefault<UToolsetRegistrySettings>();
	ToolsetRegistry = UE::ToolsetRegistry::FToolsetRegistry(
		Settings->BlockedNames,
		Settings->AllowedNames);
	UToolsetRegistry::RegisterToolsetClass(UAgentSkillToolset::StaticClass());
}

void UToolsetRegistrySubsystem::Deinitialize()
{
	UToolsetRegistry::UnregisterToolsetClass(UAgentSkillToolset::StaticClass());
	Super::Deinitialize();
}
