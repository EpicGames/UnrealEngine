// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformConverter.h"

#include "Dom/JsonValue.h"
#include "Math/Transform.h"


namespace UE::ToolsetRegistry
{
	FString FToolsetTransformConverter::GetName() const
	{
		static FString Name(TEXT("TransformConverter"));
		return Name;
	}

	bool FToolsetTransformConverter::CanConvertProperty(TNotNull<const FProperty*> Property)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (StructProperty->Struct == TBaseStructure<FTransform>::Get())
			{
				return true;
			}
		}
		return false;
	}

	TSharedPtr<FJsonObject> FToolsetTransformConverter::PropertyToJsonSchema(
		TNotNull<const FProperty*> Property)
	{
		return ToolsetStructToJsonSchema(FToolsetTransform::StaticStruct());
	}

	TSharedPtr<FJsonValue> FToolsetTransformConverter::PropertyToDefault(
		TNotNull<const FProperty*> Property, const FString& DefaultString)
	{
		FTransform DefaultTransform = FTransform::Identity;
		if (!DefaultString.IsEmpty())
		{
			Property->ImportText_Direct(*DefaultString, &DefaultTransform, nullptr, PPF_None);
		}
		FToolsetTransform ToolsetTransform = FromUE(DefaultTransform);
		return MakeShared<FJsonValueObject>(
			ToolsetStructToJsonData(FToolsetTransform::StaticStruct(), &ToolsetTransform));
	}

	TSharedPtr<FJsonValue> FToolsetTransformConverter::PropertyToJsonData(
		TNotNull<FProperty*> Property, const void* Value)
	{
		const FTransform& Transform = *static_cast<const FTransform*>(Value);
		FToolsetTransform ToolsetTransform = FromUE(Transform);
		return MakeShared<FJsonValueObject>(
			ToolsetStructToJsonData(FToolsetTransform::StaticStruct(), &ToolsetTransform));
	}

	bool FToolsetTransformConverter::JsonDataToProperty(
		const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
		void* OutValue, UObject* Outer)
	{
		TSharedPtr<FJsonObject> JsonObject = JsonValue->AsObject();
		if (!JsonObject.IsValid())
		{
			return false;
		}
		FTransform& OutTransform = *static_cast<FTransform*>(OutValue);
		FToolsetTransform ToolsetTransform;
		if (!ToolsetJsonDataToStruct(
				JsonObject.ToSharedRef(), FToolsetTransform::StaticStruct(), &ToolsetTransform))
		{
			return false;
		}
		OutTransform = ToUE(ToolsetTransform, OutTransform);
		return true;
	}

	FTransform FToolsetTransformConverter::ToUE(
		const FToolsetTransform& Transform, const FTransform& Base)
	{
		FTransform Result = Base;
		if (Transform.Location.IsSet())
		{
			Result.SetTranslation(Transform.Location.GetValue());
		}
		if (Transform.Rotation.IsSet())
		{
			Result.SetRotation(Transform.Rotation.GetValue().Quaternion());
		}
		if (Transform.Scale.IsSet())
		{
			Result.SetScale3D(Transform.Scale.GetValue());
		}
		return Result;
	}

	FToolsetTransform FToolsetTransformConverter::FromUE(const FTransform& Transform)
	{
		FToolsetTransform Result;
		Result.Location = Transform.GetTranslation();
		Result.Rotation = Transform.GetRotation().Rotator();
		Result.Scale = Transform.GetScale3D();
		return Result;
	}
}
