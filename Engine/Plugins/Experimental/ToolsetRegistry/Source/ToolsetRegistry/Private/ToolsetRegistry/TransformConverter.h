// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolsetRegistry/ToolsetJsonConverter.h"

#include "TransformConverter.generated.h"

/// Represents a 3D transformation with optional location, rotation, and scale.
/// Unset fields mean "identity" when creating objects and "don't change" when modifying existing ones.
USTRUCT(BlueprintType, MinimalAPI)
struct FToolsetTransform
{
	GENERATED_BODY()
public:
	/// The world-space location.
	UPROPERTY(BlueprintReadWrite, Category = "Transform")
	TOptional<FVector> Location;

	/// The world-space rotation.
	UPROPERTY(BlueprintReadWrite, Category = "Transform")
	TOptional<FRotator> Rotation;

	/// The scale.
	UPROPERTY(BlueprintReadWrite, Category = "Transform")
	TOptional<FVector> Scale;
};

namespace UE::ToolsetRegistry
{
	/// Converts FTransform properties to/from JSON using optional translation, rotation, and scale
	/// fields, matching the semantics of the Python Transform struct's to_ue / from_ue methods.
	class FToolsetTransformConverter : public FToolsetJsonConverter
	{
	public:
		virtual FString GetName() const override;

		virtual bool CanConvertProperty(TNotNull<const FProperty*> Property) override;

		virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(
			TNotNull<const FProperty*> Property) override;

		virtual TSharedPtr<FJsonValue> PropertyToDefault(
			TNotNull<const FProperty*> Property, const FString& DefaultString) override;

		virtual TSharedPtr<FJsonValue> PropertyToJsonData(
			TNotNull<FProperty*> Property, const void* Value) override;

		virtual bool JsonDataToProperty(
			const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
			void* OutValue, UObject* Outer) override;

	private:
		// Equivalent to Python's to_ue: applies set fields on top of Base, leaves unset fields
		// at their Base values.
		static FTransform ToUE(const FToolsetTransform& Transform,
			const FTransform& Base = FTransform::Identity);

		// Equivalent to Python's from_ue: always populates all three fields from the transform.
		static FToolsetTransform FromUE(const FTransform& Transform);
	};
}
