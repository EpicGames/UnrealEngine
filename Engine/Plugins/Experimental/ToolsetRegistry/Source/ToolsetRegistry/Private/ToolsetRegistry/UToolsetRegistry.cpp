// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolsetRegistry/UToolsetRegistry.h"

#include "Async/Future.h"
#include "Editor.h"
#include "Misc/Variant.h"
#include "Templates/SharedPointer.h"

#include "ToolsetRegistry/FunctionLibraryToolset.h"
#include "ToolsetRegistry/Module.h"
#include "ToolsetRegistry/Toolset.h"
#include "ToolsetRegistry/ToolsetRegistry.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"
#include "ToolsetRegistry/ToolCallAsyncResultString.h"

bool UToolsetRegistry::IsAvailable()
{
	return UToolsetRegistrySubsystem::Get(TEXT("IsAvailable")).HasValue();
}

void UToolsetRegistry::RegisterToolsetClass(TSubclassOf<UToolsetDefinition> InToolsetClass)
{
	if (!InToolsetClass)
	{
		UE_LOGF(LogToolsetRegistry, Error, "Cannot register null toolset class");
		return;
	}

	if (auto ToolsetRegistrySubsystem =
			UToolsetRegistrySubsystem::Get(TEXT("IsToolsetClassRegistered"));
		ToolsetRegistrySubsystem.HasValue())
	{
		using UE::ToolsetRegistry::FFunctionLibraryToolset;
		TSharedPtr<FFunctionLibraryToolset> Toolset =
			MakeShared<FFunctionLibraryToolset>(InToolsetClass);

		FString ToolsetName = FFunctionLibraryToolset::GetToolsetClassName(InToolsetClass);
		if (!Toolset->HasValidTools())
		{
			UE_LOGF(LogToolsetRegistry, Warning,
				   "Unable to register Toolset %ls: invalid tool list", *ToolsetName);
			return;
		}

		if (ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.RegisterToolset(Toolset))
		{
			UE_LOGF(LogToolsetRegistry, Display,
				"Registering Toolset %ls", *ToolsetName);
		}
		else
		{
			UE_LOGF(LogToolsetRegistry, Warning,
				"Unable to register Toolset %ls", *ToolsetName);
		}
	}
	else
	{
		UE_LOGF(LogToolsetRegistry, Error, "AIToolsetRegistrySubsystem unavailable");
	}
}

void UToolsetRegistry::UnregisterToolsetClass(TSubclassOf<UToolsetDefinition> InToolsetClass)
{
	if (!UObjectInitialized())
	{
		return;
	}

	if (!InToolsetClass)
	{
		UE_LOGF(LogToolsetRegistry, Warning, "Attempting to unregister null toolset class");
		return;
	}

	if (auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get();
		ToolsetRegistrySubsystem.HasValue())
	{
		FString ToolsetName =
			UE::ToolsetRegistry::FFunctionLibraryToolset::GetToolsetClassName(InToolsetClass);
		TSharedPtr<UE::ToolsetRegistry::FToolset> Handler =
			ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.Find(ToolsetName);
		if (Handler)
		{
			UE_LOGF(LogToolsetRegistry, Display, "Unregistering Toolset %ls", *ToolsetName);
			ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.UnregisterToolset(Handler);
		}
	}
}

bool UToolsetRegistry::IsToolsetClassRegistered(TSubclassOf<UToolsetDefinition> InToolsetClass)
{
	if (!InToolsetClass)
	{
		return false;
	}

	if (auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get(TEXT("IsToolsetClassRegistered"));
		ToolsetRegistrySubsystem.HasValue())
	{
		FString ToolsetName =
			UE::ToolsetRegistry::FFunctionLibraryToolset::GetToolsetClassName(InToolsetClass);
		return ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.Find(ToolsetName) != nullptr;
	}
	return false;
}

bool UToolsetRegistry::IsToolsetRegistered(const FString& InToolsetName)
{
	if (auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get(TEXT("IsToolsetClassRegistered"));
		ToolsetRegistrySubsystem.HasValue())
	{
		return ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.Find(InToolsetName) != nullptr;
	}
	return false;
}

UToolCallAsyncResultString* UToolsetRegistry::ExecuteTool(
	const FString& ToolsetName, const FString& ToolName, const FString& JsonInput)
{
	TStrongObjectPtr<UToolCallAsyncResultString> AsyncResult =
		TStrongObjectPtr(NewObject<UToolCallAsyncResultString>());

	auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get(TEXT("ExecuteTool"));
	if (ToolsetRegistrySubsystem.HasError())
	{
		AsyncResult->SetError(ToolsetRegistrySubsystem.StealError());
		return AsyncResult.Get();
	}
	check(ToolsetRegistrySubsystem.HasValue());
	UE::ToolsetRegistry::FToolDescriptor ToolDescriptor;
	ToolDescriptor.ToolsetName = ToolsetName;
	ToolDescriptor.ToolName = ToolName;
	auto& ToolsetRegistry = ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry;
	ToolsetRegistry.ExecuteTool(ToolDescriptor, JsonInput).Next(
		[AsyncResult](TValueOrError<FString, FString>&& ToolResult) -> void
		{
			if (ToolResult.HasError())
			{
				AsyncResult->SetError(ToolResult.StealError());
			}
			else
			{
				AsyncResult->SetValue(ToolResult.StealValue());
			}
		});
	return AsyncResult.Get();
}

FString UToolsetRegistry::GetToolsetJsonSchema(TSubclassOf<UToolsetDefinition> InToolsetClass)
{
	if (!InToolsetClass)
	{
		UE_LOGF(LogToolsetRegistry, Warning, "Attempting to get schema of null toolset class");
		return FString();
	}

	TUniquePtr<UE::ToolsetRegistry::FFunctionLibraryToolset> Toolset =
		MakeUnique<UE::ToolsetRegistry::FFunctionLibraryToolset>(InToolsetClass);
	return Toolset ? Toolset->GetJsonSchema() : FString();
}

FString UToolsetRegistry::GetAllToolsetJsonSchemas()
{
	auto ToolsetRegistrySubsystem = UToolsetRegistrySubsystem::Get(TEXT("GetAllToolsetJsonSchemas"));
	FString SchemaString;
	if (ToolsetRegistrySubsystem.HasValue())
	{
		SchemaString = ToolsetRegistrySubsystem.GetValue()->ToolsetRegistry.GetToolsetJsonSchemas();
	}
	return SchemaString;
}

