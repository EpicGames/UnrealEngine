// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/ValueOrError.h"

#include "ToolsetRegistry/JsonValueOrError.h"

namespace UE::ToolsetRegistry::Internal
{
	// Reduces the boilerplate required to construct a completed future with a value 
	// or error 
	template<typename ValueT, typename ValueOrErrorT>
	class TValueOrErrorFutureBase
	{
	public:
		using FValueOrError = ValueOrErrorT;
		using FFuture = TFuture<FValueOrError>;

	public:
		static FFuture Make(FValueOrError&& ValueOrError)
		{
			return MakeFulfilledPromise<FValueOrError>(MoveTemp(ValueOrError)).GetFuture();
		}

		static FFuture MakeValue(ValueT&& Value)
		{
			return Make(::MakeValue(MoveTemp(Value)));
		}

		static FFuture MakeError(FString&& Error)
		{
			return Make(::MakeError(MoveTemp(Error)));
		}
	};

	template<typename ValueT>
	using TValueOrErrorFuture = TValueOrErrorFutureBase<ValueT, TValueOrError<ValueT, FString>>;

	using FStringValueOrErrorFuture = TValueOrErrorFuture<FString>;

	using FJsonValueOrErrorFuture =
		TValueOrErrorFutureBase<TSharedPtr<FJsonValue>, FJsonValueOrError>;
}