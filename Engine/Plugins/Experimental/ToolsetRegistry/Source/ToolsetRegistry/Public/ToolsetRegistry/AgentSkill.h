// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"

#include "ToolsetRegistry/ToolsetDefinition.h"

#include "AgentSkill.generated.h"

#define UE_API TOOLSETREGISTRY_API

class UBlueprintFunctionLibrary;

/// Detailed information about a specific AgentSkill.
USTRUCT(BlueprintType)
struct FAgentSkillDetails
{
	GENERATED_USTRUCT_BODY()

public:
	/// Detailed information about how to use the skill.
	UPROPERTY(BlueprintReadWrite, Category = "AgentSkill")
	FString Instructions;
};

UCLASS(Blueprintable, MinimalAPI)
class UAgentSkill : public UObject
{
	GENERATED_BODY()

public:
	/// A brief description of what the skill does or how it should be used.
	UPROPERTY(EditDefaultsOnly, Category="AgentSkill", meta=(MultiLine=true))
	FString Description;

	/// Returns detailed information about how to use this skill.
	UE_API FAgentSkillDetails GetDetails() const;

protected:
	/// Generates the final prompt text for the Agent Skill.
	UFUNCTION(BlueprintNativeEvent, Category="AgentSkill")
	FString GeneratePrompt(const FString& InitialInstructions) const;
	virtual FString GeneratePrompt_Implementation(const FString& InitialInstructions) const
	{
		return InitialInstructions;
	}

	/// Detailed instructions for how to use this skill and its associated tools.
	UPROPERTY(EditDefaultsOnly, Category = "AgentSkill", meta = (MultiLine = true))
	FString Instructions;

	friend class FAgentSkillSpec;
	friend class UAgentSkillToolset;
};

/// Provides tools for listing, reading, and creating/updating skills.
UCLASS(BlueprintType, Hidden)
class UAgentSkillToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Gets a summary of all AgentSkills in the project.
	 * @return A dictionary where the key is the Skill path and the value is a description.
	 */
	UFUNCTION(meta = (AICallable), Category="Agent Skill")
	static TMap<FString, FString> ListSkills();

	/**
	 * Returns detailed information about a specific set of AgentSkills.
	 * @param SkillPaths A list of paths to the AgentSkills to retrieve.
	 * @return A dictionary where the key is the Skill path and the value is detailed info.
	 */
	UFUNCTION(meta = (AICallable), Category = "Agent Skill")
	static TMap<FString, FAgentSkillDetails> GetSkills(const TArray<FString>& SkillPaths);

	/**
	 * Creates a new AgentSkill.
	 * This should ONLY be called after getting explicit direction or permission from the user.
	 * @param FolderPath The folder in which to create the skill. i.e. /Game/Skills/.
	 * @param AssetName The name of the skill to create i.e. MySkill.
	 * @param Description The brief description of the skill.
	 * @param Details Detailed information about how to use the skill.
	 * @return The path for the created Skill class. Empty if unsuccessful.
	 */
	UFUNCTION(meta = (AICallable), Category = "Agent Skill")
	static FString CreateSkill(
		const FString& FolderPath, const FString& AssetName, const FString& Description, const FAgentSkillDetails& Details);

	/**
	 * Updates an existing AgentSkill.
	 * This should ONLY be called after getting explicit direction or permission from the user.
	 * @param SkillPath The full path to the skill to modify i.e. /Game/Skills/MySkill.MySkill_C.
	 * @param Description The brief description of the skill.
	 * @param Details Detailed information about how to use the skill.
	 * @return True if the skill was updated.
	 */
	UFUNCTION(meta = (AICallable), Category = "Agent Skill")
	static bool UpdateSkill(
		const FString& SkillPath, const FString& Description, const FAgentSkillDetails& Details);
};

#undef UE_API
