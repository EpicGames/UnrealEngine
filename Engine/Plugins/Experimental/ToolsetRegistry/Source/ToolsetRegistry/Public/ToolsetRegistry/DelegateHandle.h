// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ToolsetRegistry
{
	// Delegate handle RAII container.
	class FDelegateHandleRaiiBase
	{
	public:
		virtual ~FDelegateHandleRaiiBase() { Reset(); }

		FDelegateHandleRaiiBase(const FDelegateHandleRaiiBase&) = delete;
		FDelegateHandleRaiiBase& operator=(const FDelegateHandleRaiiBase&) = delete;

		// Reset the handle.
		virtual void Reset() { DelegateHandle.Reset(); }

		// Get the underlying delegate handle.
		const FDelegateHandle& Get() const { return DelegateHandle; }

	protected:
		FDelegateHandleRaiiBase(FDelegateHandle&& Handle) : DelegateHandle(MoveTemp(Handle)) {}

	private:
		FDelegateHandle DelegateHandle;
	};

	// RAII container for a delegate handle.
	template<typename DelegateRegistrationT>
	class TDelegateHandleRaii : public FDelegateHandleRaiiBase
	{
	public:
		// Move the delegate handle for a registration into a container.
		TDelegateHandleRaii(DelegateRegistrationT& Registration, FDelegateHandle&& Handle) :
			FDelegateHandleRaiiBase(MoveTemp(Handle)),
			DelegateRegistration(Registration)
		{
		}

		// On destruction, unregister the handle from the delegate registration.
		virtual ~TDelegateHandleRaii() { Reset(); }

		// Unregister the handle from the delegate registration.
		virtual void Reset() override
		{
			const auto& Handle = Get();
			if (Handle.IsValid())
			{
				DelegateRegistration.Remove(Handle);
				FDelegateHandleRaiiBase::Reset();
			}
		}

	private:
		DelegateRegistrationT& DelegateRegistration;
	};

	class FDelegateHandleRaii : public TUniquePtr<FDelegateHandleRaiiBase>
	{
	public:
		using TUniquePtr<FDelegateHandleRaiiBase>::TUniquePtr;

		// Whether this references a valid handle.
		bool IsValid() const
		{
			return TUniquePtr::IsValid() && Get()->Get().IsValid();
		}

	public:
		// Create an instance of a delegate handle RAII container.
		//
		// For example:
		// auto HandleRaii = FDelegateHandleRaii::Create(
		//      OnSomeMulticastDelegate,
		//      OnSomeMulticastDelegate.AddLambda([]() -> void { /* do something */ }));
		template<typename DelegateRegistrationT>
		static FDelegateHandleRaii Create(
			DelegateRegistrationT& Registration, FDelegateHandle&& Handle)
		{
			return MakeUnique<TDelegateHandleRaii<DelegateRegistrationT>>(
				Registration, MoveTemp(Handle));
		}
	};
}