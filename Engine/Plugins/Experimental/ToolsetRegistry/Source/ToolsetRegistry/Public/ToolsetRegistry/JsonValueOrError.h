// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"

#define UE_API TOOLSETREGISTRY_API

namespace UE::ToolsetRegistry
{
	/// Base class for FValueOrError.
	using FJsonValueOrErrorBase = TValueOrError<TSharedPtr<FJsonValue>, FString>;

	/// Schema descriptor for a JSON value or error.
	struct FJsonValueOrErrorSchemaDescriptor
	{
		/// Top level description for a value or error object.
		const FString ObjectDescription;
		/// Value field name in an object of this type.
		const FString ValueFieldName;
		/// Description of the value field if it can only be null.
		const FString NullValueFieldDescription;
		/// Error field name in an object of this type.
		const FString ErrorFieldName;
		/// Description of the error field.
		const FString ErrorFieldDescription;
	};

	/// JSON value or error that can be converted to / from a JSON object that is compliant with 
	/// the schema.
	/// 
	/// ```typescript
	/// { "returnValue"?: unknown, "error"?: string }
	/// ```
	class FJsonValueOrError : public FJsonValueOrErrorBase
	{
	public:
		using FJsonValueOrErrorBase::FJsonValueOrErrorBase;

		/// Convert to a value / error JSON object.
		///
		/// @param Descriptor Describes how to map value and error fields to the returned object.
		///
		/// @returns JSON object that contains the value or error.
		UE_API TSharedRef<FJsonObject> ToJsonObject(
			const FJsonValueOrErrorSchemaDescriptor& Descriptor = DefaultDescriptor) const;

		/// Convert to a JSON string.
		///
		/// @param Descriptor Describes how to map value and error fields to the returned object.
		///
		/// @returns JSON string that contains the value or error.
		UE_API FString ToJsonString(
			const FJsonValueOrErrorSchemaDescriptor& Descriptor = DefaultDescriptor) const;

	public:
		// Default descriptor for a FJsonValueOrError.
		UE_API static const FJsonValueOrErrorSchemaDescriptor DefaultDescriptor;

		/// Convert JSON schema for a JSON value / object into one that is wrapped
		/// by an object of this type (i.e a value or error).
		///
		/// @param ValueSchema Schema of the value stored by an instance of this object.
		/// @param Descriptor Describes how to map value and error fields to the returned object.
		///
		/// @returns JSON schema for a FJsonValueOrError that has ValueSchema as the value type.
		UE_API static TSharedRef<FJsonObject> GetJsonSchema(
			TSharedPtr<FJsonObject> ValueSchema,
			const FJsonValueOrErrorSchemaDescriptor& Descriptor = DefaultDescriptor);

		/// Construct from JSON string or error. If a JSON string cannot be parsed this returns
		/// an error.
		///
		/// @param JsonStringOrError JSON string or error to convert to an instance of this object.
		///
		/// @returns Instance of FJsonValueOrError that contains either the JSON value or error
		/// from JsonStringOrError.
		UE_API static FJsonValueOrError FromJsonStringOrError(
			const TValueOrError<FString, FString>& JsonStringOrError);
	};
}  // namespace UE::ToolsetRegistry

#undef UE_API