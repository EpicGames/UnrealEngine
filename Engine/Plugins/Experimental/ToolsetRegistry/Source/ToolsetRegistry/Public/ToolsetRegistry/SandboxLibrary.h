// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SandboxedFileChangeInfo.h"

#define UE_API TOOLSETREGISTRY_API

namespace UE::ToolsetRegistry
{
	/** Provides static access to the globally active FileSandbox instance. */
	class FGlobalSandbox
	{
	public:
		/** Returns whether a sandbox is currently active. */
		static UE_API bool IsActive();

		/** Returns the name of the active sandbox, or an empty string if none is active. */
		static UE_API FString GetActiveName();

		/**
		 * Creates a new sandbox or resumes an existing one by name.
		 * If a different sandbox is already active, it is left before entering the new one.
		 *
		 * @param Name        The name of the sandbox to enter or create.
		 * @param Description Used only when creating a new sandbox; ignored if a sandbox
		 *                    with the given name already exists.
		 * @return True if the named sandbox is now active; false if leaving the current
		 *         sandbox or entering the requested sandbox failed.
		 */
		static UE_API bool Enter(const FString& Name, const FString& Description);

		/**
		 * Leaves the active sandbox without deleting it. The sandbox's files are preserved.
		 *
		 * @return True if no sandbox is active after the call.
		 */
		static UE_API bool Leave();

		/**
		 * Returns the list of files changed in the active sandbox.
		 * Returns an empty array if no sandbox is active.
		 */
		static UE_API TArray<UE::FileSandboxCore::FSandboxedFileChangeInfo> GetChanges();

		/**
		 * Persists sandbox changes to the real filesystem.
		 *
		 * @param Files Files to persist. If empty, all changes are persisted.
		 * @return True on success.
		 */
		static UE_API bool Persist(const TArray<FString>& Files);

		/**
		 * Discards all changes in the active sandbox.
		 * The sandbox remains active after the call; call Leave() to exit it.
		 *
		 * @return True on success.
		 */
		static UE_API bool Discard();

		/**
		 * Discards only the specified files in the active sandbox.
		 * To discard all changes, call Discard() instead.
		 *
		 * @param Files The files to discard. Must be non-empty; passing an empty list
		 *              is treated as a caller error and returns false to avoid silently
		 *              discarding all sandbox changes.
		 * @return True if the files were reverted; false if no sandbox is active or
		 *         Files is empty.
		 */
		static UE_API bool DiscardFiles(const TArray<FString>& Files);
	};
}

#undef UE_API
