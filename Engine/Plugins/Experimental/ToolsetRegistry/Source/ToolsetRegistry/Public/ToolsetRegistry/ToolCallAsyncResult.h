// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Containers/UnrealString.h"
#include "Delegates/DelegateCombinations.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/NotNull.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

#include "ToolCallAsyncResult.generated.h"

#define UE_API TOOLSETREGISTRY_API

/// Notified when an async tool call completes successfully or with an error.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FToolCallAsyncResultCompleted,
	UToolCallAsyncResult*, Result);

/// Base class for an asynchronous tool call result that is analogous to a promise.
///
/// Asynchronous tools should *not* use this base class directly, instead they should use an 
/// either define or use an existing derivative of this base class for asynchronous results.
///
/// Asynchronous tools return derivatives of this class to indicate a pending result.
/// When an asynchronous tool completes its operation it should perform one of the following
/// operations:
/// * If successful, use MaybeBroadcastSuccessfulCompletion() from the derived class
///   - ideally implemented in a method called `SetValue` - to notify listeners of the
///   `OnCompleted` delegate, set bIsComplete to true and set the result's value.
/// * If an error occurs, use SetError() to set the `Error` property, set bIsComplete to true
///    and notify listeners of the `OnCompleted` delegate.
///
UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class UToolCallAsyncResult : public UObject
{
	GENERATED_BODY()

public:
	/// Whether the associated tool call is complete and the result via Value and GetValueAsJson()
	/// or Error are available to read.
	///
	/// @warning Do *not* set this from C++ instead use SetError() or SetValue() if implemented by
	/// the subclass.
	UPROPERTY(BlueprintReadOnly, Category = "ToolsetRegistry")
	bool bIsComplete = false;

	/// Notified when the tool call is completes successfully or with an error.
	UPROPERTY(BlueprintAssignable, Category = "ToolsetRegistry")
	FToolCallAsyncResultCompleted OnCompleted;

	/// If this is an non-empty string, an error occurred while executing the tool that returned
	/// this instance. An empty string indicates either no error occurred or the associated tool
	/// call is not complete.
	/// 
	/// @warning Do *not* set this from C++ instead use SetError() or SetValue() if implemented by
	/// the subclass.
	UPROPERTY(BlueprintReadOnly, Category = "ToolsetRegistry")
	FString Error;

	/// Get the JSON representation of the Value. If the associated tool call is not complete or an
	/// error occurred, this returns an empty string.
	UFUNCTION(BlueprintCallable, Category = "ToolsetRegistry")
	UE_API FString GetValueAsJsonString() const;

	/// Get the JSON representation of the Value. If the associated tool call is not complete or an
	/// error occurred, this returns an empty (null) shared pointer.
	UE_API virtual TSharedPtr<FJsonValue> GetValueAsJson() const;

	/// Complete this result with an error and notify listeners of OnCompleted.
	///
	/// @returns true the result was completed with the error, false otherwise.
	///
	/// @note This is thread safe, the Error property will always be updated and OnCompleted
	/// will be signaled on the main thread.
	UFUNCTION(BlueprintCallable, Category = "ToolsetRegistry")
	UE_API bool SetError(const FString& InError);

	/// Broadcast to subscribers of the OnCompleted delegate if the result is already complete.
	///
	/// This *must* be called after subscribing to OnCompleted to ensure an event receives a
	/// notification when the result is already complete.
	///
	/// @returns true if the result was complete, false otherwise.
	///
	/// @note This is thread safe, OnCompleted will always be signaled on the main thread.
	///
	/// @remark Ideally we would override subscription methods for the OnCompleted delegate
	/// and notify subscribers there but unfortunately that isn't possible.
	UFUNCTION(BlueprintCallable, Category = "ToolsetRegistry")
	UE_API bool BroadcastOnCompletedIfComplete();

protected:
	/// Broadcast that the result is complete if it hasn't already been completed.
	///
	/// @returns true if the broadcast was executed or scheduled, false otherwise.
	///
	/// @note To make this thread safe, if this isn't called on the main thread, SetFinalState 
	/// will not be called until the next main thread update so be careful that SetFinalState does
	/// not reference any data on the stack.
	UE_API bool MaybeBroadcastCompletion(TFunction<void()>&& SetFinalState);

	/// Broadcast a successful completion with a value.
	///
	/// @returns true if the broadcast was executed or scheduled, false otherwise.
	///
	/// @note: InValue must be movable, as ValueToSet must be set on the main thread.
	template<typename ValueT>
	bool MaybeBroadcastSuccessfulCompletion(ValueT&& InValue, ValueT& ValueToSet)
	{
		return MaybeBroadcastCompletion(
			[InValue = MoveTemp(InValue), &ValueToSet]() mutable -> void
			{
				ValueToSet = MoveTemp(InValue);
			});
	}

	/// Broadcast a successful completion with no value.
	///
	/// @returns true if the broadcast was executed or scheduled, false otherwise.
	bool MaybeBroadcastSuccessfulCompletion()
	{
		return MaybeBroadcastCompletion([]() -> void {});
	}

private:
	// Thread safe access of bIsComplete.
	void SetIsComplete();
	bool GetIsComplete() const;

	// Broadcast OnCompleted and clear all listeners from the delegate.
	void BroadcastOnCompleted();

private:
	// Guards: bIsComplete.
	mutable UE::FMutex CompletedLock;

public:
	// Get the JSON schema for an async result class' value.
	static TSharedRef<FJsonObject> GetValueJsonSchema(TSubclassOf<UToolCallAsyncResult> Class);

	// Check whether a property references an async result class.
	// Returns the async result UClass if the property is an FObjectProperty whose class
	// derives from UToolCallAsyncResult, nullptr otherwise.
	static UClass* MatchesProperty(TNotNull<const FProperty*> Property);

private:
	static FString ValuePropertyName;
};

#undef UE_API