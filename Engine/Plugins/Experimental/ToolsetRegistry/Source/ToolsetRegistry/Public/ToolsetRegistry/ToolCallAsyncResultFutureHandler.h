// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Dom/JsonValue.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/ToolCallAsyncResult.h"

#include "ToolCallAsyncResultFutureHandler.generated.h"

#define UE_API TOOLSETREGISTRY_API

// Handles UToolCallAsyncResult::OnCompleted by completing the associated future.
//
// NOTE: This is tested by ToolCallAsyncResultTest.cpp.
UCLASS(MinimalAPI, NotBlueprintable, HideDropdown)
class UToolCallAsyncResultFutureHandler : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual ~UToolCallAsyncResultFutureHandler() override;

	// Resolves with the JSON result of UToolCallAsyncResult object passed to Create().
	// NOTE: The future can only be retrieved once.
	UE_API TFuture<UE::ToolsetRegistry::FJsonValueOrError> GetValueAsJson();

	// Unsubscribe from the result and cancel the pending future returned from GetValueAsJson().
	UE_API void Unsubscribe();

private:
	// Handles a result completion.
	UFUNCTION()
	void OnCompleted(UToolCallAsyncResult* Result);

	// Subscribes to result completion.
	void Subscribe(UToolCallAsyncResult* Result);

	// Unsubscribe, optionally without touching the Result as it may already be garbage collected.
	void Unsubscribe(bool bIsDestructing);

private:
	TObjectPtr<UToolCallAsyncResult> Result;
	TPromise<UE::ToolsetRegistry::FJsonValueOrError> ResultPromise;
	bool bOnCompletedHandled = false;
	bool bRetrievedFuture = false;

public:
	// Create a handler and bind the OnCompleted method to the specified result instance's
	// OnCompleted delegate.
	static UE_API TStrongObjectPtr<UToolCallAsyncResultFutureHandler> Create(
		TObjectPtr<UToolCallAsyncResult> Result);

	// Error that indicates the future was canceled.
	static UE_API const FString CanceledError;
};

#undef UE_API
