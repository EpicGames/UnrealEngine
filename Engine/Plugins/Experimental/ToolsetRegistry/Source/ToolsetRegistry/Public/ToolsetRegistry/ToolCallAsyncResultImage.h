// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolCallAsyncResult.h"
#include "ToolsetRegistry/ToolsetImage.h"

#include "ToolCallAsyncResultImage.generated.h"

// Async tool call result that completes with a ToolsetImage.
UCLASS(BlueprintType)
class UToolCallAsyncResultImage : public UToolCallAsyncResult
{
	GENERATED_BODY()

public:
	// Set the result.
	UFUNCTION(BlueprintCallable, Category = "ToolsetRegistry")
	bool SetValue(const FToolsetImage& InValue)
	{
		return MaybeBroadcastSuccessfulCompletion(FToolsetImage(InValue), Value);
	}

	// Value of the result.
	UPROPERTY(BlueprintReadOnly, Category = "ToolsetRegistry")
	FToolsetImage Value;
};

