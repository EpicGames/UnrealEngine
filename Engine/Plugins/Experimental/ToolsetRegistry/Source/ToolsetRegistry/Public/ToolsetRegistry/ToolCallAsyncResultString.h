// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolCallAsyncResult.h"	

#include "ToolCallAsyncResultString.generated.h"

#define UE_API TOOLSETREGISTRY_API

// Async tool call result that completes with a string.
UCLASS(BlueprintType)
class UToolCallAsyncResultString : public UToolCallAsyncResult
{
	GENERATED_BODY()

public:
	// Set the result.
	UFUNCTION(BlueprintCallable, Category = "ToolsetRegistry")
	bool SetValue(const FString& InValue)
	{
		return MaybeBroadcastSuccessfulCompletion(FString(InValue), Value);
	}

	// Value of the result.
	UPROPERTY(BlueprintReadOnly, Category = "ToolsetRegistry")
	FString Value;
};

#undef UE_API

