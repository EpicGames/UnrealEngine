// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

#include "ToolsetRegistry/ToolCallAsyncResult.h"

#include "ToolCallAsyncResultVoid.generated.h"

#define UE_API TOOLSETREGISTRY_API

// Async tool call result that completes with no value.
UCLASS(BlueprintType)
class UToolCallAsyncResultVoid : public UToolCallAsyncResult	
{
	GENERATED_BODY()

public:
	// Complete this result with no value and notify listeners of OnCompleted.
	UFUNCTION(BlueprintCallable, Category = "ToolsetRegistry")
	bool SetCompleted()
	{
		return MaybeBroadcastSuccessfulCompletion();
	}

	UE_API virtual TSharedPtr<FJsonValue> GetValueAsJson() const override;

public:
	// Get the JSON schema for this class's value.
	static TSharedRef<FJsonObject> GetValueJsonSchema();
};

#undef UE_API 
