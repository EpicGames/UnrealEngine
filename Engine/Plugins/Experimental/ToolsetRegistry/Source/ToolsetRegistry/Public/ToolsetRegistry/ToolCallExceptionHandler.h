// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/BlueprintExceptionInfo.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"
#include "UObject/Stack.h"

#include "ToolsetRegistry/DelegateHandle.h"

#define UE_API TOOLSETREGISTRY_API

namespace UE::ToolsetRegistry
{
	/// Register for tracking of tool call exceptions for the lifetime of this object.
	class FToolCallExceptionHandler
	{
	public:
		UE_API FToolCallExceptionHandler();
		~FToolCallExceptionHandler() = default;

		// Disallow copy and move.
		FToolCallExceptionHandler(const FToolCallExceptionHandler&) = delete;
		FToolCallExceptionHandler& operator=(const FToolCallExceptionHandler&) = delete;
		FToolCallExceptionHandler(FToolCallExceptionHandler&&) = delete;
		FToolCallExceptionHandler& operator=(FToolCallExceptionHandler&&) = delete;

		/// Description of script exceptions raised by UKismetSystemLibrary::RaiseScriptError or
		/// FBlueprintCoreDelegates::ThrowScriptException via
		/// FBlueprintCoreDelegates::OnScriptException during the lifetime of this object.
		FString GetException() const
		{
			return ScriptException;
		}

		/// Call a function within a scope where it's possible to call
		/// UKismetSystemLibrary::RaiseScriptError() without dynamically calling a UFunction.
		/// 
		/// When tools are executed via FBlueprintLibraryToolset, a stack frame is configured and
		/// the appropriate method of a class derived from UToolsetDefinition is called.
		/// In test cases it is convenient to be able to call a toolset method directly from C++
		/// but this does not result in the configuration of a Blueprints stack frame (FFrame)
		/// which results in UKismetSystemLibrary::RaiseScriptError() silently doing nothing.
		/// 
		/// This method calls the specified function within a scope that has a stack frame
		/// constructed on the calling thread such that calls to
		/// UKismetSystemLibrary::RaiseScriptError() within Func will notify this class of the
		/// errors.
		/// 
		/// @warning This is intended to be only used for testing tools derived from
		/// UToolsetDefinition.
		/// 
		/// @param Func Function to call within a test scope.
		///
		UE_API void CaptureErrorsIn(TFunction<void()>&& Func);

	private:
		// Track script exceptions raised by UKismetSystemLibrary::RaiseScriptError or
		// FBlueprintCoreDelegates::ThrowScriptException via FBlueprintCoreDelegates::OnScriptException
		// and copy into local storage.
		void OnScriptException(
			const UObject* ActiveObject, const FFrame& StackFrame, const FBlueprintExceptionInfo& Info);

		// Build an exception message for the stack frame with the specified ExceptionTypeName.
		FString GetBackupExceptionMessage(const FString& ExceptionTypeName, const FFrame& StackFrame);

		// Try to build an exception message from exception info and stack frame.
		TOptional<FString> GetBlueprintExceptionMessage(
			const FBlueprintExceptionInfo& Info, const FFrame& StackFrame);

		// Append an message to the current exception message.
		void AddToExceptionMessage(const FString& NewMessage);

		// Handler for FBlueprintCoreDelegates::OnScriptException
		FDelegateHandleRaii ScriptExceptionHandle;
		// Raised script exception message.
		FString ScriptException;
	};
}

#undef UE_API
