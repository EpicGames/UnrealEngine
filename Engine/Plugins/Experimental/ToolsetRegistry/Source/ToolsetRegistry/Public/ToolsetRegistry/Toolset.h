// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Templates/ValueOrError.h"

#define UE_API TOOLSETREGISTRY_API

namespace UE::ToolsetRegistry
{
	/// Base class for toolsets.
	class FToolset
	{
	public:
		UE_API FToolset();
		UE_API virtual ~FToolset();

		/// Executes a tool with the given name and JSON input.
		///
		/// @param ToolName Name of the tool to execute.
		/// @param JsonInput JSON argument(s) for the tool.
		///
		/// @returns A future with the JSON output or an error string if it failed.
		UE_API TFuture<TValueOrError<FString, FString>> ExecuteTool(
			const FString& ToolName, const FString& JsonInput);

		/// Get the JSON schema the enabled tools in this toolset.
		/// Returns an empty string if the toolset is disabled or has no enabled tools.
		UE_API FString GetJsonSchema() const;

		/// Returns the name of the toolset
		virtual FString GetToolsetName() const = 0;

		/// Returns the version of the toolset
		virtual FString GetToolsetVersion() const = 0;

		/// Returns a description of what this toolset does and when it should be used.
		virtual FString GetToolsetDescription() const = 0;

		/// Returns the UClass that implements this toolset, if any.
		virtual UClass* GetToolsetClass() const { return nullptr; }

		/// Returns true if the toolset is enabled.
		UE_API bool IsEnabled() const;

		/// Enable or disable the toolset.
		UE_API void SetEnabled(bool bInEnabled);

		/// Set the block and allow filter patterns. Patterns enclosed in forward slashes
		/// (e.g., /^Fake.*/) are treated as regular expressions. All other patterns are matched
		/// as case-insensitive substrings. Block always takes precedence over the allow-list.
		/// Patterns that match the toolset name control whether the toolset itself is enabled.
		/// Patterns that don't match the toolset name are applied as per-tool filters.
		/// Passing empty arrays for both clears all filters.
		UE_API void SetNameFilters(
			const TArray<FString>& BlockPatterns, const TArray<FString>& AllowPatterns);

		/// Returns true if the given tool passes the per-tool block/allow filters.
		/// Tool name should be the fully-qualified tool name (e.g., "ToolsetName.ToolName").
		UE_API bool IsToolEnabled(const FString& FullToolName) const;

		/// Returns the full names of all tools in this toolset, ignoring per-tool filters
		/// and the enabled state of the toolset.
		UE_API TArray<FString> ListToolNames() const;

	protected:
		/// Executes a tool by name. Called by ExecuteTool() after enable checks pass.
		virtual TFuture<TValueOrError<FString, FString>> ExecuteToolInternal(
			const FString& ToolName, const FString& JsonInput) = 0;

		/// Get the JSON schema for all tools in the toolset.
		virtual FString GetJsonSchemaInternal() const = 0;

	private:
		/// Whether this toolset has been explicitly enabled or disabled via SetEnabled().
		bool bEnabled = true;

		/// Full names of disabled tools. Populated eagerly by SetNameFilters.
		/// An empty set means all tools are enabled (no filters active).
		TSet<FString> DisabledTools;
	};

}

#undef UE_API
