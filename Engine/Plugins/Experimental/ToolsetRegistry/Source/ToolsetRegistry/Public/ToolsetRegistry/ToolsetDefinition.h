// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/ValueOrError.h"

#include "ToolsetDefinition.generated.h"

#define UE_API TOOLSETREGISTRY_API

// This is the common base class for Toolsets defined as UObjects.
// UFunctions that define tools in this class should be static functions 
//   and be marked with meta = (AICallable). This is used both by UHT and the runtime UToolRegistry.
// UFunctions which should be ignored by tools should be marked with meta = (AIIgnore) 
//   in order to silence errors about invalid UFunctions
UCLASS(BlueprintType, Abstract, MinimalAPI)
class UToolsetDefinition : public UObject
{
	GENERATED_BODY()

public:

	// Note that this virtual method is currently called on the class default object.
	virtual FString GetToolsetVersion() const
	{
		return FString(TEXT("1.0"));
	}

	// Returns an Error if the function cannot be added to the tool and lacks the AIIgnore metadata.
	// Otherwise, returns a Value of whether or not the function can be added to the tool.
	UE_API static TValueOrError<bool, FString> IsFunctionAICallable(const TObjectPtr<const UFunction>& Function);
};
#undef UE_API
