// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IImageWrapper.h"

#include "ToolsetImage.generated.h"

#define UE_API TOOLSETREGISTRY_API

/// The standard image format that toolsets should return.
USTRUCT(BlueprintType, MinimalAPI)
struct FToolsetImage
{
	GENERATED_BODY()
public:
	/// The format the image is encoded in.
	UPROPERTY(BlueprintReadOnly, Category = "Image")
	FString MimeType;

	/// The image data encoded as a base64 string.
	UPROPERTY(BlueprintReadOnly, Category = "Image")
	FString Data;

	/// Populates the ToolsetImage based on a raw color image.
	/// Format controls the channel layout of Bitmap in memory (default BGRA matches FColor on LE).
	UE_API bool SetFromBitmap(const TArray<FColor>& Bitmap, FIntPoint Dimensions,
		ERGBFormat Format = ERGBFormat::BGRA);

	/// Populates the ToolsetImage from a file saved on disk.
	UE_API bool SetFromFile(const FString& Path);
};

#undef UE_API