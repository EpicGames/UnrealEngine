// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Templates/ValueOrError.h"

#define UE_API TOOLSETREGISTRY_API

class FJsonObject;
class FJsonValue;

namespace UE::ToolsetRegistry
{
	/// Base class for custom conversion between FProperty and JSON.
	class FToolsetJsonConverter
	{
	public:
		UE_API FToolsetJsonConverter();
		UE_API virtual ~FToolsetJsonConverter();

		/// Returns the name of the converter
		virtual FString GetName() const = 0;

		/// Determines whether a given property type can be handled by this converter.
		virtual bool CanConvertProperty(TNotNull<const FProperty*> Property) = 0;

		/*
		 * Converts an FProperty to JSON schema.
		 * @param Property The FProperty to convert.
		 * @return JSON schema that represents the FProperty.
		 */
		virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(
			TNotNull<const FProperty*> Property) = 0;

		/*
		 * Generates a default object from the FProperty.
		 * @param Property The FProperty to convert.
		 * @param DefaultString The default value of the property as a string.
		 * @return JSON data that represents the FProperty.
		 */
		virtual TSharedPtr<FJsonValue> PropertyToDefault(
			TNotNull<const FProperty*> Property, const FString& DefaultString) = 0;

		/*
		 * Gets an FProperty on an instance to JSON data.
		 * @param Property The FProperty to convert.
		 * @param Value The object or struct to get the value from.
		 * @return JSON data that represents the value of the FProperty.
		 */
		virtual TSharedPtr<FJsonValue> PropertyToJsonData(
			TNotNull<FProperty*> Property, const void* Value) = 0;

		/*
		 * Set an FProperty on an instance from JSON data.
		 * @param JsonValue The JSON data to convert.
		 * @param Property The FProperty to apply the data to.
		 * @param OutValue The object or struct to modify.
		 * @param Outer The UObject that owns the property being set.
		 * @return True if the data was converted successfully.
		 */
		virtual bool JsonDataToProperty(
			const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
			void* OutValue, UObject* Outer) = 0;

	protected:
		/// Converts a property to JSON schema via the full ToolsetJson pipeline.
		static UE_API TSharedPtr<FJsonObject> ToolsetPropertyToJsonSchema(
			TNotNull<const FProperty*> Property);

		/// Converts a property value to JSON data via the full ToolsetJson pipeline.
		static UE_API TSharedPtr<FJsonValue> ToolsetPropertyToJsonData(
			TNotNull<FProperty*> Property, TNotNull<const void*> Value);

		/// Sets a property on an instance from JSON data via the full ToolsetJson pipeline.
		static UE_API bool ToolsetJsonDataToProperty(
			const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property,
			TNotNull<void*> OutValue);

		/// Converts a UStruct to JSON schema via the full ToolsetJson pipeline.
		static UE_API TSharedPtr<FJsonObject> ToolsetStructToJsonSchema(
			TNotNull<const UStruct*> StructDefinition, bool UserVisiblePropertiesOnly = false);
		
		/// Converts a UStruct instance to JSON data via the full ToolsetJson pipeline.
		static UE_API TSharedPtr<FJsonObject> ToolsetStructToJsonData(
			TNotNull<const UStruct*> StructDefinition, TNotNull<const void*> Struct,
			bool UserVisiblePropertiesOnly = false);
		
		/// Updates a UStruct instance from JSON data via the full ToolsetJson pipeline.
		static UE_API bool ToolsetJsonDataToStruct(
			const TSharedRef<FJsonObject>& JsonObject, TNotNull<const UStruct*> StructDefinition,
			TNotNull<void*> OutStruct);
	};
}

#undef UE_API