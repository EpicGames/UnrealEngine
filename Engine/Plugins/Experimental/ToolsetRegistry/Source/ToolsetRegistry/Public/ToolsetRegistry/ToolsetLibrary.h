// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "ToolsetRegistry/ToolsetImage.h"

#include "ToolsetLibrary.generated.h"

#define UE_API TOOLSETREGISTRY_API

/// Selects whether SetObjectProperties performs container diff-and-recurse logic or
/// emits a single ValueSet for the whole property. Named (rather than bool) so call
/// sites read clearly at the point of use.
UENUM(BlueprintType)
enum class EBypassContainerCheck : uint8
{
	/// Default. Compare incoming JSON against the live container and emit precise
	/// ArrayAdd / ArrayRemove / ArrayClear / ValueSet notifications with element indices.
	No,
	/// Skip container diff logic and emit a single ValueSet for the property.
	/// Use when the receiving object handles a top-level ValueSet instead of per-element events.
	Yes,
};

/// Provides functions that are critical to Python toolset operation but which are not available
/// in and cannot be added to any other BFPL.
UCLASS(BlueprintType, MinimalAPI)
class UToolsetLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/*
	 * Returns the properties of a struct as JSON Schema.
	 * By default only returns user-visible (Blueprint-accessible) properties.
	 *
	 * WARNING: bUserVisiblePropertiesOnly=false bypasses all visibility filtering and returns
	 * every non-deprecated property on the struct. Only use this when you know that all struct
	 * properties should be exposed. For example, DataTable row structs where every field is a
	 * data column regardless of its Blueprint visibility flags.
	 *
	 * @param Struct The struct to extract properties from.
	 * @param bUserVisiblePropertiesOnly When true (default), only Blueprint-accessible properties
	 *        are included. When false, all non-deprecated properties are returned.
	 * @return A JSON Schema formatted string that describes the properties.
	 */
	UFUNCTION(BlueprintCallable, Category = "Toolset Library")
	static UE_API FString ListStructProperties(
		const UStruct* Struct, bool bUserVisiblePropertiesOnly = true);

	/*
	 * Returns the values of the requested properties.
	 * @param Object The object from which to extract properties.
	 * @param PropertyNames The names of the properties to extract.
	 * @return A JSON formatted string that contains the values of the requested properties.
	 */
	UFUNCTION(BlueprintCallable, Category = "Toolset Library")
	static UE_API FString GetObjectProperties(const UObject* Object, const TArray<FName>& PropertyNames);

	/*
	 * Sets the values of specific properties in an Object.
	 *
	 * Partial-write semantics: writes are NOT transactional. Sibling properties whose JSON
	 * resolves cleanly are committed even when other properties (or other JSON keys within
	 * a nested struct) fail. A false return value means at least one property could not be
	 * set; it does not mean the object is unchanged. Callers that need all-or-nothing
	 * semantics should wrap the call in a transaction and roll back on failure.
	 *
	 * Example: passing {"name": "ok", "doesNotExist": 0} into a struct returns false, raises
	 * a script error naming "doesNotExist", AND leaves "name" set to "ok". Same applies to
	 * unknown keys inside nested structs, which surface via the same error path.
	 *
	 * @param Object The object to modify.
	 * @param PropertiesJson The property names and values in a JSON formatted string.
	 * @param BypassContainerCheck When Yes, container-size change detection is skipped and
	 *        every property emits a plain ValueSet. Only use this when you know your object
	 *        can handle top-level ValueSet changes instead of expecting individual
	 *        ArrayAdd/ArrayRemove/ArrayClear change events when resizing containers.
	 * @return True if every property was set successfully. False indicates at least one
	 *         failure; the object may already contain partial writes from sibling properties.
	 */
	UFUNCTION(BlueprintCallable, Category = "Toolset Library")
	static UE_API bool SetObjectProperties(UObject* Object, const FString& PropertiesJson,
		EBypassContainerCheck BypassContainerCheck = EBypassContainerCheck::No);

	/** C++-only overload that also reports which property names were successfully set. */
	static UE_API bool SetObjectProperties(UObject* Object, const FString& PropertiesJson,
		TArray<FName>& OutSetPropertyNames,
		EBypassContainerCheck BypassContainerCheck = EBypassContainerCheck::No);

	/*
	 * Returns the list of subclasses that derive from a class.
	 * Uses the Asset Registry to find native and BP subclasses.
	 * @param BaseClass The class to get subclasses from.
	 * @return The list of native and BP classes that derive from the base class.
	 */
	UFUNCTION(BlueprintCallable, Category = "Toolset Library")
	static UE_API TArray<FSoftClassPath> GetDerivedClasses(UClass* BaseClass);

	/*
	 * Returns the list of substructs that derive from a struct.
	 * Iterates all loaded UScriptStruct objects to find matching substructs.
	 * @param BaseStruct The struct to get substructs from.
	 * @return The list of loaded structs that derive from the base struct.
	 */
	UFUNCTION(BlueprintCallable, Category = "Toolset Library")
	static UE_API TArray<UScriptStruct*> GetDerivedStructs(UScriptStruct* BaseStruct);

	/*
	 * Undo the most recent transaction on the global undo stack.
	 *
	 * Companion to UKismetSystemLibrary's BeginTransaction / EndTransaction /
	 * CancelTransaction, which cover starting and finalizing transactions but
	 * stop short of actually reverting them: CancelTransaction only discards
	 * the undo-stack entry without applying the inverse, so any modifications
	 * a partial transaction made to UObjects survive. Used by
	 * programmatic.execute_tool_script to roll back scripts that error out
	 * partway through.
	 *
	 * @param bCanRedo If true, the undone transaction stays on the redo stack.
	 *        If false (default), it is removed entirely so a rolled-back
	 *        script leaves no trace on the undo history.
	 * @return True if a transaction was undone; false if there was nothing to
	 *         undo or the editor is in a state that prevents undo (e.g. saving
	 *         a package, GC running, or no GEditor).
	 */
	UFUNCTION(BlueprintCallable, Category = "Toolset Library")
	static UE_API bool UndoTransaction(bool bCanRedo = false);

	/*
	 * Returns the number of entries currently undoable on the global undo stack
	 * (queue length minus undo count, i.e. entries above the redo split).
	 *
	 * Snapshotting this around a BeginTransaction / EndTransaction pair lets
	 * a caller tell whether the pair actually committed a record: a transient
	 * transaction (no UObject changes) is silently dropped by UTransBuffer::End,
	 * leaving the count unchanged.
	 *
	 * @return The active undo count, or 0 when there is no editor transaction
	 *         buffer available (e.g. no GEditor).
	 */
	UFUNCTION(BlueprintCallable, Category = "Toolset Library")
	static UE_API int32 GetActiveUndoCount();
};

#undef UE_API
