// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Misc/Variant.h"
#include "Templates/SharedPointer.h"

#include "ToolsetRegistry/Toolset.h"
#include "ToolsetRegistry/ToolsetJsonConverter.h"

#define UE_API TOOLSETREGISTRY_API

namespace UE::ToolsetRegistry
{

	// Descriptor for a tool within a toolset.
	struct FToolDescriptor
	{
		FString ToolsetName;
		FString ToolName;

		UE_API static TValueOrError<FToolDescriptor, FString> FromString(
			const FString& ToolsetAndToolName, bool LogErrors = false);
	};

	// Registry for AI toolsets. Allows registering, unregistering, finding, and executing
	// tools from toolsets.
	class FToolsetRegistry {
	public:
		UE_API FToolsetRegistry(
			const TArray<FString>& InBlockedNames = {},
			const TArray<FString>& InAllowedNames = {});
		UE_API ~FToolsetRegistry();

		// Add a pattern to the block-list. Any toolset or tool whose name matches the pattern
		// will be treated as non-existent. Patterns enclosed in forward slashes
		// (e.g., /^Fake.*/) are treated as regular expressions. All other patterns are matched
		// as case-insensitive substrings.
		// Block always takes precedence over the allow-list.
		UE_API void AddBlockedName(const FString& Name);

		// Remove a pattern from the block-list. No-op if the pattern is not present.
		UE_API void RemoveBlockedName(const FString& Name);

		// Returns the current block-list patterns.
		UE_API const TArray<FString>& GetBlockedNames() const;

		// Add a pattern to the allow-list. Any toolset or tool whose name does not match these
		// patterns will be treated as non-existent. Patterns enclosed in forward slashes
		// (e.g., /^Fake.*/) are treated as regular expressions. All other patterns are matched
		// as case-insensitive substrings.
		// Block always takes precedence over the allow-list.
		UE_API void AddAllowedName(const FString& Name);

		// Remove a pattern from the allow-list. No-op if the pattern is not present.
		UE_API void RemoveAllowedName(const FString& Name);

		// Returns the current allow-list patterns.
		UE_API const TArray<FString>& GetAllowedNames() const;

		// Register a toolset handler. Returns true if registration was successful.
		UE_API bool RegisterToolset(TSharedPtr<FToolset> ToolsetHandler);

		// Unregister a toolset handler. Returns true if unregistration was successful.
		UE_API bool UnregisterToolset(TSharedPtr<FToolset> ToolsetHandler);

		// Find a registered toolset handler by name, and optionally log errors.
		UE_API TSharedPtr<FToolset> Find(
			const FString& ToolsetName, bool LogErrors = false, FString* ErrorMessage = nullptr);

		// Execute a tool from a registered toolset handler.
		// 
		// If there is an error in attempting to execute a tool, such as a tool not existing,
		// the future will contain an error string.
		UE_API TFuture<TValueOrError<FString, FString>> ExecuteTool(
			const FString& ToolName, const FString& JsonInput);

		// Execute a tool from a registered toolset handler.
		// 
		// If there is an error in attempting to execute a tool, such as a tool not existing,
		// the future will contain an error string.
		UE_API TFuture<TValueOrError<FString, FString>> ExecuteTool(
			const FToolDescriptor& ToolDescriptor, const FString& JsonInput);

		// Iterate all registered toolsets.
		UE_API void ForEachToolset(
			TFunctionRef<void(const FString& Name, const FToolset& Toolset)> Callback) const;

		// Get the JSON schemas for all registered toolsets.
		// Returns a JSON list concatenating all registered Toolset schemas.
		UE_API FString GetToolsetJsonSchemas() const;
		
		// Delegate to be called when the toolset registry changes.
		DECLARE_MULTICAST_DELEGATE(FOnToolsetRegistryChanged);

		// Get the delegate for toolset registry changes.
		FOnToolsetRegistryChanged& OnToolsetRegistered()
		{
			return OnToolsetRegistryChanged;
		}

		// Register a custom JSON converter. Returns true if registration was successful.
		UE_API bool RegisterConverter(TSharedPtr<FToolsetJsonConverter> JsonConverter);

		// Unregister a custom JSON converter. Returns true if unregistration was successful.
		UE_API bool UnregisterConverter(TSharedPtr<FToolsetJsonConverter> JsonConverter);

		// Returns the converter that can handle the specified property, if any.
		UE_API TSharedPtr<FToolsetJsonConverter> GetConverterForProperty(
			TNotNull<const FProperty*> Property);

	private:
		// Calls SetNameFilters on every registered toolset, then broadcasts
		// OnToolsetRegistryChanged. Called after any change to BlockedNames or AllowedNames.
		void ApplyFiltersToAllToolsets();

		// Block-list pattern strings.
		TArray<FString> BlockedNames;

		// Allow-list pattern strings.
		TArray<FString> AllowedNames;

		// The set of registered toolset handlers.
		TMap<FString, TSharedPtr<FToolset>> ToolsetHandlers;

		// The set of registered converters.
		TMap<FString, TSharedPtr<FToolsetJsonConverter>> JsonConverters;

		// Callbacks to be called when the toolset registry changes.
		FOnToolsetRegistryChanged OnToolsetRegistryChanged;
	};

}

#undef UE_API