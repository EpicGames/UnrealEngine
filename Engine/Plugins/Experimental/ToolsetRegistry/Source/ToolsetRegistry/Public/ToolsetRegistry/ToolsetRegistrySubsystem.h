// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EditorSubsystem.h"
#include "Engine/DeveloperSettings.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "UObject/ObjectPtr.h"

#include "ToolsetRegistry.h"

#include "ToolsetRegistrySubsystem.generated.h"

#define UE_API TOOLSETREGISTRY_API

class FSubsystemCollectionBase;

UCLASS(config=EditorPerProjectUserSettings, defaultconfig, MinimalAPI, meta=(DisplayName="Toolset Registry"))
class UToolsetRegistrySettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	virtual FName GetCategoryName() const override { return TEXT("Plugins"); }

	// Toolsets and tools whose names match any of these patterns are hidden. Patterns
	// enclosed in forward slashes (e.g., /^Fake.*/) are treated as regular expressions;
	// all others are matched as case-insensitive substrings. Block takes precedence over
	// ToolsetAllowedNames.
	UPROPERTY(config, EditAnywhere, Category="Toolsets", meta=(DisplayName="ToolsetBlockedNames"))
	TArray<FString> BlockedNames;

	// When non-empty, only toolsets and tools whose names match one of these patterns are
	// visible. Same pattern syntax as ToolsetBlockedNames. Block takes precedence over
	// this list.
	UPROPERTY(config, EditAnywhere, Category="Toolsets", meta=(DisplayName="ToolsetAllowedNames"))
	TArray<FString> AllowedNames;

	// AgentSkills whose paths match any of these patterns are hidden from ListSkills
	// and GetSkills. Patterns enclosed in forward slashes (e.g., /^Foo.*/) are treated
	// as regular expressions; all others are matched as case-insensitive substrings
	// against the skill's full path. Block takes precedence over AgentSkillAllowedNames.
	UPROPERTY(config, EditAnywhere, Category="Agent Skills")
	TArray<FString> AgentSkillBlockedNames;

	// When non-empty, only AgentSkills whose paths match one of these patterns are
	// visible. Same pattern syntax as AgentSkillBlockedNames. Block takes precedence
	// over this list.
	UPROPERTY(config, EditAnywhere, Category="Agent Skills")
	TArray<FString> AgentSkillAllowedNames;

	// Classes whose instances must be rejected by UToolsetLibrary::SetObjectProperties.
	// Matched against the object's class and every parent class. Pattern syntax matches
	// ToolsetBlockedNames (forward-slash-wrapped patterns are regex; others are
	// case-insensitive substring).
	UPROPERTY(config, EditAnywhere, Category="SetObjectProperties Block List")
	TArray<FString> SetObjectPropertiesBlockedClasses;

	// Class/property pairs that must be rejected by UToolsetLibrary::SetObjectProperties.
	// Each pattern is matched against "ClassName.PropertyName" for every class in the
	// target object's inheritance chain. Use a bare property name with a regex
	// (e.g., /\.RootComponent$/) to match the property on every class. Same pattern
	// syntax as SetObjectPropertiesBlockedClasses.
	UPROPERTY(config, EditAnywhere, Category="SetObjectProperties Block List")
	TArray<FString> SetObjectPropertiesBlockedProperties;
};

UCLASS(BlueprintType, MinimalAPI)
class UToolsetRegistrySubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	// Get the toolset registry subsystem.  Optionally log a warning if passed a non-empty string
	// (the warning message will start with the passed-in string).
	UE_API static TValueOrError<TObjectPtr<UToolsetRegistrySubsystem>, FString> Get(
		const FString& WarningMessage = FString());

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	// The set of registered toolset handlers.
	UE::ToolsetRegistry::FToolsetRegistry ToolsetRegistry;
};

#undef UE_API
