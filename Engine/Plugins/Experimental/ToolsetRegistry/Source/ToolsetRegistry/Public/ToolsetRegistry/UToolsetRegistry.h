// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"
#include "ToolsetRegistry/ToolsetDefinition.h"
#include "ToolsetRegistry/ToolCallAsyncResultString.h"

#include "UToolsetRegistry.generated.h"

#define UE_API TOOLSETREGISTRY_API

// Blueprint-accessible wrapper to allow queries against the AI Toolset Registry.
UCLASS(MinimalAPI)
class UToolsetRegistry : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Whether the toolset registry is available.
	// For example, this will return false if the editor is not available.
	UFUNCTION(BlueprintCallable, Category="ToolsetRegistry")
	static UE_API bool IsAvailable();

	// Register a Blueprint function library class as a toolset.
	UFUNCTION(BlueprintCallable, Category="ToolsetRegistry")
	static UE_API void RegisterToolsetClass(TSubclassOf<UToolsetDefinition> InToolsetClass);

	// Unregister a Blueprint function library class as a toolset.
	UFUNCTION(BlueprintCallable, Category="ToolsetRegistry")
	static UE_API void UnregisterToolsetClass(TSubclassOf<UToolsetDefinition> InToolsetClass);

 	// Return whether a toolset class is already registered.
	UFUNCTION(BlueprintPure, Category="ToolsetRegistry")
	static UE_API bool IsToolsetClassRegistered(TSubclassOf<UToolsetDefinition> InToolsetClass);

 	// Return whether a toolset is already registered by name.
	UFUNCTION(BlueprintPure, Category="ToolsetRegistry")
	static UE_API bool IsToolsetRegistered(const FString& InToolsetName);

	// Execute a registered tool given a toolset and tool name as if it was being executed by an 
	// AI assistant. If successful, the result's value is a JSON string containing the tool's
	// return value. If a failure occurs, the result will contain an error.
	UFUNCTION(BlueprintCallable, Category="ToolsetRegistry")
	static UE_API UToolCallAsyncResultString* ExecuteTool(
		const FString& ToolsetName, const FString& ToolName, const FString& JsonInput);

	// Get toolset JSON schema for a toolset class.
	// Meant for use while testing.
	UFUNCTION(BlueprintPure, Category="ToolsetRegistry")
	static UE_API FString GetToolsetJsonSchema(TSubclassOf<UToolsetDefinition> InToolsetClass);

	// Get JSON schemas for all registered toolsets.
	UFUNCTION(BlueprintPure, Category="ToolsetRegistry")
	static UE_API FString GetAllToolsetJsonSchemas();
};

#undef UE_API
