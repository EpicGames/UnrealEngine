// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ToolsetRegistry : ModuleRules
{
	public ToolsetRegistry(ReadOnlyTargetRules Target) : base(Target)
	{
		// For IWYU audits..
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs; // ..keep
		IWYUSupport = IWYUSupport.Full;  // ..keep, can be None/Minimal/Full
		bUseUnity = true; // ..toggle as needed, use false for IWYU audit


		PublicDefinitions.Add("WITH_AIASSISTANT_EPIC_INTERNAL=1");


		PublicIncludePaths.AddRange(
			new string[] {
			}
			);


		PrivateIncludePaths.AddRange(
			new string[] {
			}
			);


		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"CoreUObject",
				"DeveloperSettings",
				"EditorFramework",
				"EditorScriptingUtilities",
				"EditorSubsystem",
				"Engine",
				"Json",
				"JsonUtilities",
				"JsonUtilitiesEditor",
				"Kismet",
				"PythonScriptPlugin",
				"UnrealEd",
				"FileSandboxCore",
			}
			);


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
