{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Toolset Registry",
	"Description": "",
	"Category": "Other",
	"EditorOnly": true,
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"NoRedist": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "ToolsetRegistry",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "FileSandbox",
			"Enabled": true
		}
	]
}